{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Draw Debug Library",
	"Description": "A library of common debug drawing functions.",
	"Category": "Gameplay",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "DrawDebugLibrary",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
	],
	"Plugins": [
		{
			"Name": "RigVM",
			"Enabled": true
		}
	]
}