// Copyright Epic Games, Inc. All Rights Reserved.

#include "DrawDebugLibrary.h"

#include "Animation/TrajectoryTypes.h"
#include "PrimitiveDrawInterface.h"
#include "SkeletalDebugRendering.h"
#include "HitProxies.h"
#include "DrawDebugHelpers.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimInstanceProxy.h"
#include "VisualLogger/VisualLogger.h"
#include "Components/SkinnedMeshComponent.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVM/Public/RigVMCore/RigVMDrawInterface.h"
#include "Engine/StaticMesh.h"
#include "Debug/DebugDrawService.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DrawDebugLibrary)

void FDrawDebugLibraryModule::StartupModule()
{
	static UScriptStruct* const AllowedStructTypes[] =
	{
		FDrawDebugPointStyle::StaticStruct(),
		FDrawDebugLineStyle::StaticStruct(),
		FDrawDebugArrowSettings::StaticStruct(),
		FDrawDebugChairSettings::StaticStruct(),
		FDrawDebugBackpackSettings::StaticStruct(),
		FDrawDebugBriefcaseSettings::StaticStruct(),
		FDrawDebugDoorSettings::StaticStruct(),
		FDrawDebugSkeletonSettings::StaticStruct(),
		FDrawDebugStringSettings::StaticStruct(),
		FDrawDebugGraphAxesSettings::StaticStruct(),
		FDebugDrawer::StaticStruct(),
	};

	FRigVMRegistry& RigVMRegistry = FRigVMRegistry::Get();
	RigVMRegistry.RegisterStructTypes(AllowedStructTypes);
}

void FDrawDebugLibraryModule::ShutdownModule() {}
	
IMPLEMENT_MODULE(FDrawDebugLibraryModule, DrawDebugLibrary)

namespace UE::DrawDebugLibrary::Private
{
	static bool bRigVMComponentSpace = false;

	FAutoConsoleVariableRef CVarRigVMComponentSpace(
		TEXT("r.DrawDebugLibrary.RigVMComponentSpace"),
		bRigVMComponentSpace,
		TEXT("When enabled, the RigVM Drawer draws in Component Space where possible rather than World Space by default."),
		ECVF_Default);

	static FQuat FindOrientation(const FVector Start, const FVector End)
	{
		return (End - Start).GetSafeNormal(UE_KINDA_SMALL_NUMBER, FVector::ForwardVector).Rotation().Quaternion();
	}

	static FVector HermiteInterpolate(const FVector P0, const FVector P1, const FVector V0, const FVector V1, const float X)
	{
		const double W1 = 3 * X * X - 2 * X * X * X;
		const double W2 = X * X * X - 2 * X * X + X;
		const double W3 = X * X * X - X * X;

		return W1 * (P1 - P0) + W2 * V0 + W3 * V1 + P0;
	}

	static inline double ComputeMonotoneVelocity(const double D0, const double D1)
	{
		return FMath::Sign(D0) != FMath::Sign(D1) ? 0.0 :
			FMath::Clamp(
				(D0 + D1) / 2.0,
				FMath::Max(-3.0 * FMath::Abs(D0), -3.0 * FMath::Abs(D1)),
				FMath::Min(+3.0 * FMath::Abs(D0), +3.0 * FMath::Abs(D1)));
	}

	static inline FVector ComputeMonotoneVelocity(const FVector D0, const FVector D1)
	{
		return FVector(
			ComputeMonotoneVelocity(D0.X, D1.X),
			ComputeMonotoneVelocity(D0.Y, D1.Y),
			ComputeMonotoneVelocity(D0.Z, D1.Z));
	}

	static inline FVector InterpolateCubicMonoStart(const FVector P1, const FVector P2, const FVector P3, const float Alpha)
	{
		const FVector V1 = P2 - P1;
		const FVector V2 = ComputeMonotoneVelocity(P2 - P1, P3 - P2);
		return HermiteInterpolate(P1, P2, V1, V2, Alpha);
	}

	static inline FVector InterpolateCubicMonoEnd(const FVector P0, const FVector P1, const FVector P2, const float Alpha)
	{
		const FVector V1 = ComputeMonotoneVelocity(P1 - P0, P2 - P1);
		const FVector V2 = P2 - P1;
		return HermiteInterpolate(P1, P2, V1, V2, Alpha);
	}

	static inline FVector InterpolateCubicMono(const FVector P0, const FVector P1, const FVector P2, const FVector P3, const float Alpha)
	{
		const FVector V1 = ComputeMonotoneVelocity(P1 - P0, P2 - P1);
		const FVector V2 = ComputeMonotoneVelocity(P2 - P1, P3 - P2);
		return HermiteInterpolate(P1, P2, V1, V2, Alpha);
	}

	static inline FVector InterpolateCubicStart(const FVector P1, const FVector P2, const FVector P3, const float Alpha)
	{
		const FVector V1 = (P2 - P1);
		const FVector V2 = ((P2 - P1) + (P3 - P2)) / 2;
		return HermiteInterpolate(P1, P2, V1, V2, Alpha);
	}

	static inline FVector InterpolateCubicEnd(const FVector P0, const FVector P1, const FVector P2, const float Alpha)
	{
		const FVector V1 = ((P1 - P0) + (P2 - P1)) / 2;
		const FVector V2 = (P2 - P1);
		return HermiteInterpolate(P1, P2, V1, V2, Alpha);
	}

	static inline FVector InterpolateCubic(const FVector P0, const FVector P1, const FVector P2, const FVector P3, const float Alpha)
	{
		const FVector V1 = ((P1 - P0) + (P2 - P1)) / 2;
		const FVector V2 = ((P2 - P1) + (P3 - P2)) / 2;
		return HermiteInterpolate(P1, P2, V1, V2, Alpha);
	}

	// Line fonts from here: https://theorangeduck.com/page/debug-draw-text-lines

	static const uint8 MonoStringLineNums[94] = {
		 2,  6,  4, 10, 15, 16,  4,  5,  5,  3,  2,  3,  1,  1,  1, 13,  3,  6,
		11,  4,  8, 13,  2, 14, 15,  2,  4,  2,  2,  2,  7, 28,  3, 12, 10,  9,
		 4,  3, 11,  3,  3,  5,  4,  2,  4,  3, 12,  7, 15,  8, 12,  2,  6,  2,
		 4,  2,  3,  3,  3,  1,  3,  2,  1,  1, 11, 10,  8, 10, 12,  5, 26,  6,
		 4,  6,  4,  3, 11,  6, 10,  9,  9,  5, 10,  5,  6,  2,  4,  2,  4,  3,
		10,  1, 10,  7
	};

	static const int32 MonoStringLineOffsets[94] = {
		  0,   2,   8,  12,  22,  37,  53,  57,  62,  67,  70,  72,  75,  76,
		 77,  78,  91,  94, 100, 111, 115, 123, 136, 138, 152, 167, 169, 173,
		175, 177, 179, 186, 214, 217, 229, 239, 248, 252, 255, 266, 269, 272,
		277, 281, 283, 287, 290, 302, 309, 324, 332, 344, 346, 352, 354, 358,
		360, 363, 366, 369, 370, 373, 375, 376, 377, 388, 398, 406, 416, 428,
		433, 459, 465, 469, 475, 479, 482, 493, 499, 509, 518, 527, 532, 542,
		547, 553, 555, 559, 561, 565, 568, 578, 579, 589
	};

	static const int32 MonoStringLineSegments[596] = {
		0x701F3C1F, 0x2A1F281F, 0x59135206, 0x63175913, 0x6D116317, 0x59305223,
		0x63345930, 0x6D2E6334, 0x563A5608, 0x3B373B05, 0x6A172610, 0x26276A2E,
		0x27222909, 0x2D2F2722, 0x35342D2F, 0x3E313534, 0x510F3E31, 0x590B510F,
		0x620E590B, 0x681A620E, 0x6730681A, 0x19187425, 0x26077038, 0x6D096105,
		0x70156D09, 0x671D7015, 0x5B1B671D, 0x55125B1B, 0x59085512, 0x61055908,
		0x3D263122, 0x40323D26, 0x373A4032, 0x2B38373A, 0x252F2B38, 0x2925252F,
		0x31222925, 0x2538590E, 0x55265F2B, 0x610D590E, 0x6A12610D, 0x6E1C6A12,
		0x6B266E1C, 0x662A6B26, 0x5F2B662A, 0x3F085526, 0x2A282519, 0x290F2519,
		0x2F09290F, 0x37062F09, 0x3F083706, 0x39332A28, 0x46343933, 0x6E1D6621,
		0x5D226621, 0x541A5D22, 0x5210541A, 0x5C1B722B, 0x49165C1B, 0x37164916,
		0x271B3716, 0x102B271B, 0x5C247214, 0x49295C24, 0x37294929, 0x27243729,
		0x10142724, 0x70204620, 0x65324E0D, 0x650E4F32, 0x291F5A1F, 0x42074238,
		0x2423301D, 0x191E2423, 0x1310191E, 0x422D4212, 0x2A1F2C1F, 0x70301A0C,
		0x5A35360A, 0x4A08360A, 0x5F0D4A08, 0x66145F0D, 0x6A206614, 0x652C6A20,
		0x5A35652C, 0x46375A35, 0x31324637, 0x2B2B3132, 0x261F2B2B, 0x2B13261F,
		0x360A2B13, 0x2735270D, 0x6A222722, 0x5E0C6A22, 0x270B2735, 0x502F270B,
		0x5C30502F, 0x672A5C30, 0x6B1D672A, 0x630D6B1D, 0x2625270C, 0x2F2F2625,
		0x38332F2F, 0x40313833, 0x49224031, 0x522C4922, 0x5C2F522C, 0x652B5C2F,
		0x6A1D652B, 0x670E6A1D, 0x49164922, 0x37053739, 0x262B6A2B, 0x6A233705,
		0x6A2B6A23, 0x680F6830, 0x4A0F680F, 0x4A274A0F, 0x270D261F, 0x2E2E261F,
		0x3A322E2E, 0x442F3A32, 0x4A27442F, 0x671E6930, 0x5E12671E, 0x4E0C5E12,
		0x460B4E0C, 0x360C460B, 0x460B4D1D, 0x2914360C, 0x25212914, 0x2A2E2521,
		0x35352A2E, 0x43333535, 0x4C294333, 0x4D1D4C29, 0x6935690A, 0x26156935,
		0x67125C0C, 0x6B206712, 0x662E6B20, 0x5C32662E, 0x522E5C32, 0x3E0E522E,
		0x350B3E0E, 0x2813350B, 0x261F2813, 0x2A2D261F, 0x32332A2D, 0x3C313233,
		0x53103C31, 0x5C0C5310, 0x2920270D, 0x322D2920, 0x3E32322D, 0x4C333E32,
		0x4C334529, 0x5F304C33, 0x67295F30, 0x6A1E6729, 0x66126A1E, 0x5E0B6612,
		0x550A5E0B, 0x4A0D550A, 0x44144A0D, 0x421E4414, 0x4529421E, 0x531F561F,
		0x291F2C1F, 0x531F561F, 0x23242E1E, 0x181E2324, 0x1311181E, 0x420D262E,
		0x5C2D420D, 0x4C354C0B, 0x380A3835, 0x42312610, 0x5C114231, 0x3B1B4A1B,
		0x4D2A4A1B, 0x59304D2A, 0x622D5930, 0x6B22622D, 0x6E146B22, 0x271B2A1B,
		0x0F25122D, 0x0E1C0F25, 0x12100E1C, 0x1F071210, 0x34041F07, 0x4B063404,
		0x5E0C4B06, 0x6C185E0C, 0x70246C18, 0x6D2F7024, 0x65366D2F, 0x563A6536,
		0x483B563A, 0x3538483B, 0x2B333538, 0x292E2B33, 0x2C29292E, 0x37282C29,
		0x2D203728, 0x4F2B3728, 0x53274F2B, 0x54225327, 0x4E195422, 0x41154E19,
		0x32144115, 0x2B173214, 0x291B2B17, 0x2D20291B, 0x6A1F2606, 0x26396A1F,
		0x3832380D, 0x6A0D260D, 0x672B6A0D, 0x490D4924, 0x260D292C, 0x3032292C,
		0x39353032, 0x40323935, 0x49244032, 0x502E4924, 0x5932502E, 0x61305932,
		0x672B6130, 0x26272A35, 0x29182627, 0x30102918, 0x390C3010, 0x460A390C,
		0x560C460A, 0x6112560C, 0x691E6112, 0x6A27691E, 0x66356A27, 0x690A260A,
		0x6920690A, 0x260A2620, 0x632C6920, 0x5933632C, 0x49375933, 0x39344937,
		0x2E2E3934, 0x26202E2E, 0x260F2632, 0x690F260F, 0x6932690F, 0x490F4931,
		0x6A102510, 0x6A326A10, 0x4810482F, 0x48354823, 0x28354835, 0x6A276635,
		0x67196A27, 0x5D0E6719, 0x4A085D0E, 0x3C094A08, 0x300E3C09, 0x2819300E,
		0x26252819, 0x28352625, 0x6B0A240A, 0x4936490A, 0x24366B36, 0x2633260C,
		0x6933690C, 0x261F691F, 0x692D690E, 0x312D692D, 0x2A25312D, 0x261B2A25,
		0x2A0D261B, 0x6A0D250D, 0x49176A32, 0x25334917, 0x490D4917, 0x26126A12,
		0x26342612, 0x6A0B2507, 0x3F1F6A0B, 0x6A353F1F, 0x25386A35, 0x6B0C250C,
		0x24356B0C, 0x6A352435, 0x662D6A20, 0x5E33662D, 0x4C385E33, 0x39354C38,
		0x2C2E3935, 0x26202C2E, 0x28142620, 0x340A2814, 0x4707340A, 0x580A4707,
		0x6613580A, 0x6A206613, 0x6A0D260D, 0x420D4226, 0x4A304226, 0x56354A30,
		0x632F5635, 0x6A1F632F, 0x6A0D6A1F, 0x122B153A, 0x1E1F122B, 0x251F1E1F,
		0x2B2D251F, 0x3C362B2D, 0x49383C36, 0x59354938, 0x662D5935, 0x6A20662D,
		0x64116A20, 0x560A6411, 0x4707560A, 0x360A4707, 0x2A12360A, 0x251F2A12,
		0x6A0D250D, 0x6A206A0D, 0x652C6A20, 0x5A31652C, 0x512F5A31, 0x512F4822,
		0x480D4822, 0x25344822, 0x6A20682F, 0x67136A20, 0x610D6713, 0x5A0B610D,
		0x520E5A0B, 0x4A1A520E, 0x412E4A1A, 0x3D32412E, 0x35343D32, 0x2D303534,
		0x26222D30, 0x280A2622, 0x69396908, 0x261F691F, 0x340B6A0B, 0x6A353435,
		0x2A10340B, 0x251F2A10, 0x292B251F, 0x3435292B, 0x251F6A06, 0x6A3A251F,
		0x260E6A06, 0x511F260E, 0x2634511F, 0x6A382634, 0x6A352508, 0x6A0A2536,
		0x40206A06, 0x6A394020, 0x25204020, 0x6936690B, 0x26096936, 0x26362609,
		0x7117712C, 0x0F177117, 0x0F2C0F17, 0x1933700E, 0x71287112, 0x0F287128,
		0x0F130F28, 0x6A1F4B0C, 0x4B336A1F, 0x0F3C0F03, 0x681D7013, 0x560F5920,
		0x4E312531, 0x562C4E31, 0x5920562C, 0x3F153F31, 0x370D3F15, 0x2F0B370D,
		0x26112F0B, 0x251B2611, 0x2825251B, 0x32312825, 0x290D6F0D, 0x251C290D,
		0x2728251C, 0x36332728, 0x43353633, 0x50314335, 0x562C5031, 0x5924562C,
		0x551B5924, 0x4B0D551B, 0x59255632, 0x55185925, 0x4A0F5518, 0x3F0C4A0F,
		0x330F3F0C, 0x2819330F, 0x26252819, 0x29322625, 0x25317031, 0x27213731,
		0x25192721, 0x29112519, 0x310C2911, 0x3E0A310C, 0x4E0E3E0A, 0x55154E0E,
		0x59215515, 0x53315921, 0x4035400B, 0x4B334035, 0x532F4B33, 0x5728532F,
		0x59205728, 0x56155920, 0x4C0D5615, 0x400B4C0D, 0x320D400B, 0x2815320D,
		0x25242815, 0x28332524, 0x661B261B, 0x6E397027, 0x6D20661B, 0x70276D20,
		0x4F074F37, 0x58285837, 0x512E5828, 0x591C5828, 0x5411591C, 0x4A0D5411,
		0x3D124A0D, 0x391E3D12, 0x370E3D12, 0x310C370E, 0x2B0F310C, 0x28142B0F,
		0x282A2814, 0x220C2814, 0x2532282A, 0x21352532, 0x1C362135, 0x16331C36,
		0x122D1633, 0x0E1E122D, 0x110F0E1E, 0x150A110F, 0x1B09150A, 0x220C1B09,
		0x3C2A391E, 0x47303C2A, 0x512E4730, 0x250D700D, 0x25324F32, 0x5519490D,
		0x59235519, 0x562D5923, 0x4F32562D, 0x2634260D, 0x57212621, 0x570E5721,
		0x6A1F6D1F, 0x6A296D29, 0x572B570D, 0x1E2B572B, 0x0E16110A, 0x11220E16,
		0x1E2B1122, 0x250F700F, 0x411B410F, 0x5932411B, 0x2533411B, 0x2633260D,
		0x6F212621, 0x6F0E6F21, 0x25085808, 0x25204C20, 0x25365136, 0x56134808,
		0x59195613, 0x551E5919, 0x4C20551E, 0x572A4C20, 0x5930572A, 0x57345930,
		0x51365734, 0x580D250D, 0x25324E32, 0x54174A0D, 0x59245417, 0x542F5924,
		0x4E32542F, 0x500E3E09, 0x591D500E, 0x552E591D, 0x4834552E, 0x35354834,
		0x292C3535, 0x251F292C, 0x2A11251F, 0x330B2A11, 0x3E09330B, 0x570D0E0D,
		0x571C490D, 0x5925571C, 0x54305925, 0x43355430, 0x31324335, 0x28283132,
		0x251D2828, 0x290D251D, 0x0F315531, 0x591F5531, 0x5312591F, 0x400A5312,
		0x2E0D400A, 0x27132E0D, 0x251B2713, 0x2B26251B, 0x37312B26, 0x570F250F,
		0x571F480F, 0x5928571F, 0x55325928, 0x49355532, 0x5921572F, 0x56145921,
		0x4D0F5614, 0x44134D0F, 0x3B2B4413, 0x36303B2B, 0x30323630, 0x292C3032,
		0x251D292C, 0x280D251D, 0x69173317, 0x2A1B3317, 0x26272A1B, 0x27342627,
		0x58345806, 0x2F0D580D, 0x26325932, 0x28132F0D, 0x251B2813, 0x2A26251B,
		0x34322A26, 0x251F5809, 0x5835251F, 0x25115806, 0x4A1F2511, 0x24314A1F,
		0x58382431, 0x5833250B, 0x580D2534, 0x171A5935, 0x1113171A, 0x10071113,
		0x241F580A, 0x5831580D, 0x260C5831, 0x2634260C, 0x6F24712F, 0x671D6F24,
		0x4E1D671D, 0x48194E1D, 0x430C4819, 0x3F19430C, 0x371E3F19, 0x1B1E371E,
		0x13221B1E, 0x0F2F1322, 0x0F1F7C1F, 0x6F1A710F, 0x67216F1A, 0x4E216721,
		0x48254E21, 0x43324825, 0x3F254332, 0x37203F25, 0x1B203720, 0x131C1B20,
		0x0F0F131C, 0x450A3D08, 0x4A13450A, 0x431E4A13, 0x3B27431E, 0x392D3B27,
		0x3F35392D, 0x47373F35
	};

	static const char StringLineNums[94] = {
		 5,  2,  4, 20, 21, 22,  6,  7,  7,  3,  2,  7,  1,  4,  1, 16,  3, 13,
		14,  3, 16, 22,  2, 28, 22,  8, 11,  2,  2,  2, 17, 37,  3, 18, 17, 12,
		 4,  3, 19,  3,  1,  9,  3,  2,  4,  3, 20, 10, 21, 11, 19,  2,  9,  2,
		 4,  2,  3,  3,  3,  1,  3,  2,  1,  6, 14, 14, 13, 14, 16,  5, 19,  7,
		 5,  8,  3,  1, 13,  7, 16, 14, 14,  5, 16,  5,  7,  2,  4,  2,  6,  3,
		10,  1, 10,  8
	};

	static const char StringLineAdvs[94] = {
		40, 64, 84, 80, 96, 96, 40, 48, 48, 64, 88, 40, 88, 40, 64, 80, 80, 80,
		80, 80, 80, 80, 80, 80, 80, 40, 40, 96, 104, 96, 72, 108, 72, 84, 84, 84,
		76, 72, 84, 88, 32, 64, 84, 68, 96, 88, 88, 84, 88, 84, 80, 64, 88, 72,
		96, 80, 72, 80, 44, 56, 44, 64, 64, 40, 76, 76, 72, 76, 72, 48, 76, 76,
		32, 40, 68, 32, 120, 76, 76, 76, 76, 52, 68, 48, 76, 64, 88, 68, 64, 68,
		56, 32, 56, 96
	};

	static const int StringLineOffsets[94] = {
		  0,   5,   7,  11,  31,  52,  74,  80,  87,  94,  97,  99, 106, 107,
		111, 112, 128, 131, 144, 158, 161, 177, 199, 201, 229, 251, 259, 270,
		272, 274, 276, 293, 330, 333, 351, 368, 380, 384, 387, 406, 409, 410,
		419, 422, 424, 428, 431, 451, 461, 482, 493, 512, 514, 523, 525, 529,
		531, 534, 537, 540, 541, 544, 546, 547, 553, 567, 581, 594, 608, 624,
		629, 648, 655, 660, 668, 671, 672, 685, 692, 708, 722, 736, 741, 757,
		762, 769, 771, 775, 777, 783, 786, 796, 797, 807
	};

	static const int StringLineSegments[815] = {
		0x3C147414, 0x24102814, 0x20142410, 0x24182014, 0x28142418, 0x58107410,
		0x58307430, 0x18187024, 0x1830703C, 0x50485010, 0x3844380C, 0x10248430,
		0x703C6844, 0x7430703C, 0x74207430, 0x70147420, 0x680C7014, 0x600C680C,
		0x5810600C, 0x54145810, 0x501C5414, 0x4834501C, 0x443C4834, 0x4040443C,
		0x38444040, 0x2C443844, 0x243C2C44, 0x2030243C, 0x20202030, 0x24142020,
		0x2C0C2414, 0x201C7444, 0x6C287420, 0x64286C28, 0x5C246428, 0x581C5C24,
		0x5814581C, 0x600C5814, 0x680C600C, 0x7010680C, 0x74187010, 0x74207418,
		0x383C3C44, 0x3038383C, 0x28383038, 0x20402838, 0x20482040, 0x24502048,
		0x2C542450, 0x34542C54, 0x3C4C3454, 0x3C443C4C, 0x2C3C404C, 0x24342C3C,
		0x202C2434, 0x201C202C, 0x2414201C, 0x28102414, 0x300C2810, 0x380C300C,
		0x4010380C, 0x44144010, 0x54304414, 0x58345430, 0x60385834, 0x68386038,
		0x70346838, 0x742C7034, 0x7024742C, 0x68207024, 0x60206820, 0x54246020,
		0x482C5424, 0x204C482C, 0x70106C14, 0x74147010, 0x70187414, 0x68187018,
		0x60146818, 0x5C106014, 0x701C7820, 0x6014701C, 0x4C106014, 0x3C104C10,
		0x28143C10, 0x181C2814, 0x1020181C, 0x70147810, 0x601C7014, 0x4C20601C,
		0x3C204C20, 0x281C3C20, 0x1814281C, 0x10101814, 0x44207420, 0x5034680C,
		0x500C6834, 0x282C602C, 0x44484410, 0x20142418, 0x24102014, 0x28142410,
		0x24182814, 0x1C182418, 0x14141C18, 0x10101414, 0x44484410, 0x24102814,
		0x20142410, 0x24182014, 0x28142418, 0x20087430, 0x70187424, 0x64107018,
		0x500C6410, 0x440C500C, 0x3010440C, 0x24183010, 0x20242418, 0x202C2024,
		0x2438202C, 0x30402438, 0x44443040, 0x50444444, 0x64405044, 0x70386440,
		0x742C7038, 0x7424742C, 0x68206418, 0x742C6820, 0x202C742C, 0x64106010,
		0x6C146410, 0x70186C14, 0x74207018, 0x74307420, 0x70387430, 0x6C3C7038,
		0x64406C3C, 0x5C406440, 0x543C5C40, 0x4834543C, 0x200C4834, 0x2044200C,
		0x74407414, 0x54287440, 0x54345428, 0x503C5434, 0x4C40503C, 0x40444C40,
		0x38444044, 0x2C403844, 0x24382C40, 0x202C2438, 0x2020202C, 0x24142020,
		0x28102414, 0x300C2810, 0x3C0C7434, 0x3C483C0C, 0x20347434, 0x7414743C,
		0x50107414, 0x54145010, 0x58205414, 0x582C5820, 0x5438582C, 0x4C405438,
		0x40444C40, 0x38444044, 0x2C403844, 0x24382C40, 0x202C2438, 0x2020202C,
		0x24142020, 0x28102414, 0x300C2810, 0x703C6840, 0x7430703C, 0x74287430,
		0x701C7428, 0x6414701C, 0x50106414, 0x3C105010, 0x2C143C10, 0x241C2C14,
		0x2028241C, 0x202C2028, 0x2438202C, 0x2C402438, 0x38442C40, 0x3C443844,
		0x48403C44, 0x50384840, 0x542C5038, 0x5428542C, 0x501C5428, 0x4814501C,
		0x3C104814, 0x201C7444, 0x7444740C, 0x70147420, 0x68107014, 0x60106810,
		0x58146010, 0x541C5814, 0x502C541C, 0x4C38502C, 0x44404C38, 0x3C444440,
		0x30443C44, 0x28403044, 0x243C2840, 0x2030243C, 0x20202030, 0x24142020,
		0x28102414, 0x300C2810, 0x3C0C300C, 0x44103C0C, 0x4C184410, 0x50244C18,
		0x54345024, 0x583C5434, 0x6040583C, 0x68406040, 0x703C6840, 0x7430703C,
		0x74207430, 0x4C3C5840, 0x44344C3C, 0x40284434, 0x40244028, 0x44184024,
		0x4C104418, 0x580C4C10, 0x5C0C580C, 0x68105C0C, 0x70186810, 0x74247018,
		0x74287424, 0x70347428, 0x683C7034, 0x5840683C, 0x44405840, 0x303C4440,
		0x2434303C, 0x20282434, 0x20202028, 0x24142020, 0x2C102414, 0x54105814,
		0x50145410, 0x54185014, 0x58145418, 0x24102814, 0x20142410, 0x24182014,
		0x28142418, 0x54105814, 0x50145410, 0x54185014, 0x58145418, 0x20142418,
		0x24102014, 0x28142410, 0x24182814, 0x1C182418, 0x14141C18, 0x10101414,
		0x44106850, 0x20504410, 0x50585010, 0x38583810, 0x44506810, 0x20104450,
		0x640C600C, 0x6C10640C, 0x70146C10, 0x741C7014, 0x742C741C, 0x7034742C,
		0x6C387034, 0x643C6C38, 0x5C3C643C, 0x54385C3C, 0x50345438, 0x48245034,
		0x3C244824, 0x24202824, 0x20242420, 0x24282024, 0x28242428, 0x5C445448,
		0x603C5C44, 0x6030603C, 0x5C286030, 0x58245C28, 0x4C205824, 0x40204C20,
		0x38244020, 0x342C3824, 0x3438342C, 0x38403438, 0x40443840, 0x40446048,
		0x38444044, 0x344C3844, 0x3454344C, 0x3C5C3454, 0x48603C5C, 0x50604860,
		0x5C5C5060, 0x64585C5C, 0x6C506458, 0x70486C50, 0x743C7048, 0x7430743C,
		0x70247430, 0x6C1C7024, 0x64146C1C, 0x5C106414, 0x500C5C10, 0x440C500C,
		0x3810440C, 0x30143810, 0x281C3014, 0x2424281C, 0x20302424, 0x20402030,
		0x20047424, 0x20447424, 0x3C383C10, 0x20107410, 0x74347410, 0x70407434,
		0x6C447040, 0x64486C44, 0x5C486448, 0x54445C48, 0x50405444, 0x4C345040,
		0x4C344C10, 0x48404C34, 0x44444840, 0x3C484444, 0x30483C48, 0x28443048,
		0x24402844, 0x20342440, 0x20102034, 0x68446048, 0x703C6844, 0x7434703C,
		0x74247434, 0x701C7424, 0x6814701C, 0x60106814, 0x540C6010, 0x400C540C,
		0x3410400C, 0x2C143410, 0x241C2C14, 0x2024241C, 0x20342024, 0x243C2034,
		0x2C44243C, 0x34482C44, 0x20107410, 0x742C7410, 0x7038742C, 0x68407038,
		0x60446840, 0x54486044, 0x40485448, 0x34444048, 0x2C403444, 0x24382C40,
		0x202C2438, 0x2010202C, 0x20107410, 0x74447410, 0x4C304C10, 0x20442010,
		0x20107410, 0x74447410, 0x4C304C10, 0x68446048, 0x703C6844, 0x7434703C,
		0x74247434, 0x701C7424, 0x6814701C, 0x60106814, 0x540C6010, 0x400C540C,
		0x3410400C, 0x2C143410, 0x241C2C14, 0x2024241C, 0x20342024, 0x243C2034,
		0x2C44243C, 0x34482C44, 0x40483448, 0x40484034, 0x20107410, 0x20487448,
		0x4C484C10, 0x20107410, 0x34307430, 0x282C3430, 0x2428282C, 0x20202428,
		0x20182020, 0x24102018, 0x280C2410, 0x3408280C, 0x3C083408, 0x20107410,
		0x3C107448, 0x20485024, 0x20107410, 0x20402010, 0x20107410, 0x20307410,
		0x20307450, 0x20507450, 0x20107410, 0x20487410, 0x20487448, 0x701C7424,
		0x6814701C, 0x60106814, 0x540C6010, 0x400C540C, 0x3410400C, 0x2C143410,
		0x241C2C14, 0x2024241C, 0x20342024, 0x243C2034, 0x2C44243C, 0x34482C44,
		0x404C3448, 0x544C404C, 0x6048544C, 0x68446048, 0x703C6844, 0x7434703C,
		0x74247434, 0x20107410, 0x74347410, 0x70407434, 0x6C447040, 0x64486C44,
		0x58486448, 0x50445848, 0x4C405044, 0x48344C40, 0x48104834, 0x701C7424,
		0x6814701C, 0x60106814, 0x540C6010, 0x400C540C, 0x3410400C, 0x2C143410,
		0x241C2C14, 0x2024241C, 0x20342024, 0x243C2034, 0x2C44243C, 0x34482C44,
		0x404C3448, 0x544C404C, 0x6048544C, 0x68446048, 0x703C6844, 0x7434703C,
		0x74247434, 0x18483030, 0x20107410, 0x74347410, 0x70407434, 0x6C447040,
		0x64486C44, 0x5C486448, 0x54445C48, 0x50405444, 0x4C345040, 0x4C104C34,
		0x20484C2C, 0x703C6844, 0x7430703C, 0x74207430, 0x70147420, 0x680C7014,
		0x600C680C, 0x5810600C, 0x54145810, 0x501C5414, 0x4834501C, 0x443C4834,
		0x4040443C, 0x38444040, 0x2C443844, 0x243C2C44, 0x2030243C, 0x20202030,
		0x24142020, 0x2C0C2414, 0x20207420, 0x743C7404, 0x38107410, 0x2C143810,
		0x241C2C14, 0x2028241C, 0x20302028, 0x243C2030, 0x2C44243C, 0x38482C44,
		0x74483848, 0x20247404, 0x20247444, 0x201C7408, 0x201C7430, 0x20447430,
		0x20447458, 0x2044740C, 0x200C7444, 0x4C247404, 0x20244C24, 0x4C247444,
		0x200C7444, 0x7444740C, 0x2044200C, 0x78107820, 0x18107810, 0x18201810,
		0x20307408, 0x78207810, 0x18207820, 0x18101820, 0x70205C0C, 0x5C347020,
		0x18401800, 0x70147418, 0x68107014, 0x60106810, 0x5C146010, 0x60185C14,
		0x64146018, 0x203C583C, 0x54344C3C, 0x582C5434, 0x5820582C, 0x54185820,
		0x4C105418, 0x400C4C10, 0x380C400C, 0x2C10380C, 0x24182C10, 0x20202418,
		0x202C2020, 0x2434202C, 0x2C3C2434, 0x20107410, 0x54184C10, 0x58205418,
		0x582C5820, 0x5434582C, 0x4C3C5434, 0x40404C3C, 0x38404040, 0x2C3C3840,
		0x24342C3C, 0x202C2434, 0x2020202C, 0x24182020, 0x2C102418, 0x54344C3C,
		0x582C5434, 0x5820582C, 0x54185820, 0x4C105418, 0x400C4C10, 0x380C400C,
		0x2C10380C, 0x24182C10, 0x20202418, 0x202C2020, 0x2434202C, 0x2C3C2434,
		0x203C743C, 0x54344C3C, 0x582C5434, 0x5820582C, 0x54185820, 0x4C105418,
		0x400C4C10, 0x380C400C, 0x2C10380C, 0x24182C10, 0x20202418, 0x202C2020,
		0x2434202C, 0x2C3C2434, 0x403C400C, 0x483C403C, 0x5038483C, 0x54345038,
		0x582C5434, 0x5820582C, 0x54185820, 0x4C105418, 0x400C4C10, 0x380C400C,
		0x2C10380C, 0x24182C10, 0x20202418, 0x202C2020, 0x2434202C, 0x2C3C2434,
		0x74207428, 0x70187420, 0x64147018, 0x20146414, 0x58245808, 0x183C583C,
		0x0C38183C, 0x08340C38, 0x042C0834, 0x0420042C, 0x08180420, 0x54344C3C,
		0x582C5434, 0x5820582C, 0x54185820, 0x4C105418, 0x400C4C10, 0x380C400C,
		0x2C10380C, 0x24182C10, 0x20202418, 0x202C2020, 0x2434202C, 0x2C3C2434,
		0x20107410, 0x541C4810, 0x5824541C, 0x58305824, 0x54385830, 0x483C5438,
		0x203C483C, 0x7010740C, 0x74147010, 0x78107414, 0x740C7810, 0x20105810,
		0x70187414, 0x741C7018, 0x7818741C, 0x74147818, 0x14185818, 0x08141418,
		0x040C0814, 0x0404040C, 0x20107410, 0x30105838, 0x203C4020, 0x20107410,
		0x20105810, 0x541C4810, 0x5824541C, 0x58305824, 0x54385830, 0x483C5438,
		0x203C483C, 0x5448483C, 0x58505448, 0x585C5850, 0x5464585C, 0x48685464,
		0x20684868, 0x20105810, 0x541C4810, 0x5824541C, 0x58305824, 0x54385830,
		0x483C5438, 0x203C483C, 0x54185820, 0x4C105418, 0x400C4C10, 0x380C400C,
		0x2C10380C, 0x24182C10, 0x20202418, 0x202C2020, 0x2434202C, 0x2C3C2434,
		0x38402C3C, 0x40403840, 0x4C3C4040, 0x54344C3C, 0x582C5434, 0x5820582C,
		0x04105810, 0x54184C10, 0x58205418, 0x582C5820, 0x5434582C, 0x4C3C5434,
		0x40404C3C, 0x38404040, 0x2C3C3840, 0x24342C3C, 0x202C2434, 0x2020202C,
		0x24182020, 0x2C102418, 0x043C583C, 0x54344C3C, 0x582C5434, 0x5820582C,
		0x54185820, 0x4C105418, 0x400C4C10, 0x380C400C, 0x2C10380C, 0x24182C10,
		0x20202418, 0x202C2020, 0x2434202C, 0x2C3C2434, 0x20105810, 0x4C144010,
		0x541C4C14, 0x5824541C, 0x58305824, 0x54344C38, 0x58285434, 0x581C5828,
		0x5410581C, 0x4C0C5410, 0x44104C0C, 0x40184410, 0x3C2C4018, 0x38343C2C,
		0x30383834, 0x2C383038, 0x24342C38, 0x20282434, 0x201C2028, 0x2410201C,
		0x2C0C2410, 0x30147414, 0x24183014, 0x20202418, 0x20282020, 0x58245808,
		0x30105810, 0x24143010, 0x201C2414, 0x2028201C, 0x24302028, 0x303C2430,
		0x203C583C, 0x20205808, 0x20205838, 0x201C580C, 0x201C582C, 0x203C582C,
		0x203C584C, 0x2038580C, 0x200C5838, 0x20205808, 0x20205838, 0x10182020,
		0x08101018, 0x04080810, 0x04040408, 0x200C5838, 0x5838580C, 0x2038200C,
		0x741C7824, 0x6C18741C, 0x50186C18, 0x48145018, 0x44104814, 0x40144410,
		0x38184014, 0x1C183818, 0x141C1C18, 0x1024141C, 0x10107810, 0x74187810,
		0x6C1C7418, 0x501C6C1C, 0x4820501C, 0x44244820, 0x40204424, 0x381C4020,
		0x1C1C381C, 0x14181C1C, 0x10101418, 0x4810400C, 0x4C184810, 0x4C204C18,
		0x40384C20, 0x3C404038, 0x3C483C40, 0x40503C48, 0x48544050
	};

	static ELogVerbosity::Type LogVerbosityFromDrawDebugLogVerbosity(const EDrawDebugLogVerbosity Verbosity)
	{
		switch (Verbosity)
		{
		case EDrawDebugLogVerbosity::Fatal: return ELogVerbosity::Fatal;
		case EDrawDebugLogVerbosity::Error: return ELogVerbosity::Error;
		case EDrawDebugLogVerbosity::Warning: return ELogVerbosity::Warning;
		case EDrawDebugLogVerbosity::Display: return ELogVerbosity::Display;
		case EDrawDebugLogVerbosity::Log: return ELogVerbosity::Log;
		case EDrawDebugLogVerbosity::Verbose: return ELogVerbosity::Verbose;
		case EDrawDebugLogVerbosity::VeryVerbose: return ELogVerbosity::VeryVerbose;
		default: checkNoEntry(); return ELogVerbosity::Log;
		}
	}

	static void DrawPointsToCanvas(FCanvas* Canvas, const float Scale, const TArrayView<const FVector> Points, const FLinearColor Color, const float Thickness)
	{
		if (!Canvas) { return; }

		const int32 Num = Points.Num();

		for (int32 Idx = 0; Idx < Num; Idx++)
		{
			const FVector2D Point = Scale * FVector2D(Points[Idx].X, Points[Idx].Y);

			FCanvasLineItem LineItem(Point, Point);
			LineItem.LineThickness = Thickness == 0.0f ? 1.0f : Thickness;
			LineItem.SetColor(Color);
			Canvas->DrawItem(LineItem);
		}
	}

	static void DrawLinesToCanvas(FCanvas* Canvas, const float Scale, const TArrayView<const TPair<FVector, FVector>> LineSegments, const FLinearColor Color, const float Thickness)
	{
		if (!Canvas) { return; }

		const int32 Num = LineSegments.Num();

		for (int32 Idx = 0; Idx < Num; Idx++)
		{
			const FVector2D Start = Scale * FVector2D(LineSegments[Idx].Key.X, LineSegments[Idx].Key.Y);
			const FVector2D End = Scale * FVector2D(LineSegments[Idx].Value.X, LineSegments[Idx].Value.Y);

			FCanvasLineItem LineItem(Start, End);
			LineItem.LineThickness = Thickness == 0.0f ? 1.0f : Thickness;
			LineItem.SetColor(Color);
			Canvas->DrawItem(LineItem);
		}
	}

	static void DrawTextToCanvas(FCanvas* Canvas, const FVector2D Location, const FText& Text, const FLinearColor Color, const FDrawDebugCanvasTextSettings& Settings)
	{
		if (!Canvas) { return; }

		UFont* Font = nullptr;

		if (Settings.Font)
		{
			Font = Settings.Font;
		}
		else if (GEngine)
		{
			Font = Settings.bMonospaced ? GEngine->GetMonospaceFont() : GEngine->GetSmallFont();
		}

		FCanvasTextItem TextItem(Location, Text, Font, Color);
		TextItem.Scale.X = Settings.WidthScale;
		TextItem.Scale.Y = Settings.HeightScale;
		if (Settings.bShadow)
		{
			TextItem.EnableShadow(Settings.ShadowColor, Settings.ShadowOffset);
		}
		else
		{
			TextItem.DisableShadow();
		}
		TextItem.bOutlined = Settings.bOutline;
		TextItem.OutlineColor = Settings.OutlineColor;
		TextItem.bCentreX = Settings.bCenterHorizontally;
		TextItem.bCentreY = Settings.bCenterVertically;
		Canvas->DrawItem(TextItem);
	}
}

FDebugDrawerCanvasBuffer::~FDebugDrawerCanvasBuffer()
{
	if (Delegate.IsValid())
	{
		AsyncTask(ENamedThreads::GameThread, [LocalDelegate = Delegate]() { UDebugDrawService::Unregister(LocalDelegate); });
		Delegate.Reset();
	}
}

void FDebugDrawerCanvasBuffer::Flush(FCanvas* Canvas)
{
	if (Canvas)
	{
		const int32 EntryNum = PrimitiveNums.Num();

		int32 PointOffset = 0;
		int32 LineOffset = 0;
		int32 TextOffset = 0;

		for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
		{
			if (PrimitiveTypes[EntryIdx] == 0)
			{
				UE::DrawDebugLibrary::Private::DrawPointsToCanvas(Canvas, Scale, MakeArrayView(Points).Slice(PointOffset, PrimitiveNums[EntryIdx]), Colors[EntryIdx], Thicknesses[EntryIdx]);
				PointOffset += PrimitiveNums[EntryIdx];
			}
			else if (PrimitiveTypes[EntryIdx] == 1)
			{
				UE::DrawDebugLibrary::Private::DrawLinesToCanvas(Canvas, Scale, MakeArrayView(LineSegments).Slice(LineOffset, PrimitiveNums[EntryIdx]), Colors[EntryIdx], Thicknesses[EntryIdx]);
				LineOffset += PrimitiveNums[EntryIdx];
			}
			else if (PrimitiveTypes[EntryIdx] == 2)
			{
				UE::DrawDebugLibrary::Private::DrawTextToCanvas(Canvas, Texts[TextOffset].Key, Texts[TextOffset].Value, Colors[EntryIdx], TextSettings[TextOffset]);
				TextOffset += PrimitiveNums[EntryIdx];
			}
			else
			{
				checkNoEntry();
			}
		}

		check(PointOffset == Points.Num());
		check(LineOffset == LineSegments.Num());
		check(TextOffset == Texts.Num());
		check(TextOffset == TextSettings.Num());
	}

	PrimitiveTypes.Reset();
	PrimitiveNums.Reset();
	Colors.Reset();
	Thicknesses.Reset();
	Points.Reset();
	LineSegments.Reset();
	Texts.Reset();
	TextSettings.Reset();
}

FDebugDrawer::FDebugDrawer() = default;
FDebugDrawer::~FDebugDrawer() = default;

FDebugDrawer::FDebugDrawer(UObject* InObject) : FDebugDrawer()
{
	World = InObject ? InObject->GetWorld() : nullptr;
}

FDebugDrawer::FDebugDrawer(UWorld* InWorld) : FDebugDrawer()
{
	World = InWorld;
}

FDebugDrawer::FDebugDrawer(UAnimInstance* InAnimInstance) : FDebugDrawer()
{
	AnimInstance = InAnimInstance;
}

FDebugDrawer::FDebugDrawer(FAnimInstanceProxy* InAnimInstanceProxy) : FDebugDrawer()
{
	AnimInstanceProxy = InAnimInstanceProxy;
}

FDebugDrawer::FDebugDrawer(FPrimitiveDrawInterface* InPrimitiveDrawInterface) : FDebugDrawer()
{
	PDI = InPrimitiveDrawInterface;
}

FDebugDrawer::FDebugDrawer(FRigVMDrawInterface* InRigVMDrawInterface, const USceneComponent* InRigVMComponent) : FDebugDrawer()
{
	RigVM = InRigVMDrawInterface;
	RigVMComponent = InRigVMComponent;
}

FDebugDrawer::FDebugDrawer(FCanvas* InCanvas) : FDebugDrawer()
{
	Canvas = InCanvas;
}

FDebugDrawer::FDebugDrawer(UCanvas* InCanvas) : FDebugDrawer()
{
	Canvas = InCanvas ? InCanvas->Canvas : nullptr;
}

FDebugDrawer::FDebugDrawer(const TSharedPtr<FDebugDrawerCanvasBuffer>& InCanvasBuffer)
{
	CanvasBuffer = InCanvasBuffer;
}

FDebugDrawer FDebugDrawer::MakeDebugDrawer()
{
	return FDebugDrawer();
}

FDebugDrawer FDebugDrawer::MakeDebugDrawer(UObject* Object)
{
	return MakeDebugDrawer(Object ? Object->GetWorld() : nullptr);
}

FDebugDrawer FDebugDrawer::MakeDebugDrawer(UWorld* World)
{
	return FDebugDrawer(World);
}

FDebugDrawer FDebugDrawer::MakeDebugDrawer(UAnimInstance* AnimInstance)
{
	return FDebugDrawer(AnimInstance);
}

FDebugDrawer FDebugDrawer::MakeDebugDrawer(FAnimInstanceProxy* AnimInstanceProxy)
{
	return FDebugDrawer(AnimInstanceProxy);
}

FDebugDrawer FDebugDrawer::MakeDebugDrawer(FPrimitiveDrawInterface* PrimitiveDrawInterface)
{
	return FDebugDrawer(PrimitiveDrawInterface);
}

FDebugDrawer FDebugDrawer::MakeDebugDrawer(FRigVMDrawInterface* InRigVMDrawInterface, const USceneComponent* InRigVMComponent)
{
	return FDebugDrawer(InRigVMDrawInterface, InRigVMComponent);
}

FDebugDrawer FDebugDrawer::MakeDebugDrawer(FCanvas* InCanvas, const float InCanvasScale)
{
	FDebugDrawer Drawer;
	Drawer.Canvas = InCanvas;
	Drawer.CanvasScale = InCanvasScale;
	return Drawer;
}

FDebugDrawer FDebugDrawer::MakeDebugDrawer(UCanvas* InCanvas, const float InCanvasScale)
{
	return MakeDebugDrawer(InCanvas ? InCanvas->Canvas : nullptr, InCanvasScale);
}

FDebugDrawer FDebugDrawer::MakeDebugDrawer(const TSharedPtr<FDebugDrawerCanvasBuffer>& CanvasBuffer)
{
	FDebugDrawer Drawer;
	Drawer.CanvasBuffer = CanvasBuffer;
	return Drawer;
}

FDebugDrawer FDebugDrawer::MakeVisualLoggerDebugDrawer(
	const UObject* Object,
	const FName Category, 
	const EDrawDebugLogVerbosity Verbosity, 
	const bool bDrawToScene,
	const bool bDrawToSceneWhileRecording)
{
	FDebugDrawer Drawer;
	Drawer.VisualLoggerObject = Object;
	Drawer.VisualLoggerCategory = Category;
	Drawer.VisualLoggerVerbosity = UE::DrawDebugLibrary::Private::LogVerbosityFromDrawDebugLogVerbosity(Verbosity);
	Drawer.bVisualLoggerDrawToScene = bDrawToScene;
	Drawer.bVisualLoggerDrawToSceneWhileRecording = bDrawToSceneWhileRecording;
	return Drawer;
}

FDebugDrawer FDebugDrawer::MakeVisualLoggerDebugDrawer(
	const UObject* Object,
	const FLogCategoryBase& Category, 
	const ELogVerbosity::Type Verbosity, 
	const bool bDrawToScene,
	const bool bDrawToSceneWhileRecording)
{
	FDebugDrawer Drawer;
	Drawer.VisualLoggerObject = Object;
	Drawer.VisualLoggerCategory = Category.GetCategoryName();
	Drawer.VisualLoggerVerbosity = Verbosity;
	Drawer.bVisualLoggerDrawToScene = bDrawToScene;
	Drawer.bVisualLoggerDrawToSceneWhileRecording = bDrawToSceneWhileRecording;
	return Drawer;
}

FDebugDrawer FDebugDrawer::MakeScreenDebugDrawer(const float Scale)
{
	if (!IsInGameThread())
	{
		return FDebugDrawer();
	}

	TSharedPtr<FDebugDrawerCanvasBuffer> Buffer = MakeShared<FDebugDrawerCanvasBuffer>();

	const FDelegateHandle Delegate = UDebugDrawService::Register(TEXT("GameplayDebug"),
		FDebugDrawDelegate::CreateLambda([BufferWeak = Buffer.ToWeakPtr()](UCanvas* Canvas, APlayerController* Controller)
			{
				if (Canvas)
				{
					if (TSharedPtr<FDebugDrawerCanvasBuffer> Buffer = BufferWeak.Pin())
					{
						Buffer->Flush(Canvas->Canvas);
					}
				}
			}));

	if (Delegate.IsValid())
	{
		FDebugDrawer Drawer;
		Drawer.CanvasBuffer = Buffer;
		Drawer.CanvasBuffer->Delegate = Delegate;
		Drawer.CanvasBuffer->Scale = Scale;
		return Drawer;
	}

	return FDebugDrawer();
}

FDebugDrawer FDebugDrawer::MakeMergedDebugDrawer(const TArrayView<const FDebugDrawer> DebugDrawers)
{
	FDebugDrawer Drawer;

	for (const FDebugDrawer& SubDrawer : DebugDrawers)
	{
		if (SubDrawer.PDI) { Drawer.PDI = SubDrawer.PDI; }
		if (SubDrawer.RigVM) { Drawer.RigVM = SubDrawer.RigVM; Drawer.RigVMComponent = SubDrawer.RigVMComponent; }
		if (SubDrawer.World) { Drawer.World = SubDrawer.World; }
		if (SubDrawer.AnimInstance) { Drawer.AnimInstance = SubDrawer.AnimInstance; }
		if (SubDrawer.AnimInstanceProxy) { Drawer.AnimInstanceProxy = SubDrawer.AnimInstanceProxy; }
		if (SubDrawer.Canvas) { Drawer.Canvas = SubDrawer.Canvas; Drawer.CanvasScale = SubDrawer.CanvasScale; }
		if (SubDrawer.CanvasBuffer) { Drawer.CanvasBuffer = SubDrawer.CanvasBuffer; }
		if (SubDrawer.VisualLoggerObject)
		{
			Drawer.VisualLoggerObject = SubDrawer.VisualLoggerObject;
			Drawer.VisualLoggerCategory = SubDrawer.VisualLoggerCategory;
			Drawer.VisualLoggerVerbosity = SubDrawer.VisualLoggerVerbosity;
			Drawer.bVisualLoggerDrawToScene = SubDrawer.bVisualLoggerDrawToScene;
			Drawer.bVisualLoggerDrawToSceneWhileRecording = SubDrawer.bVisualLoggerDrawToSceneWhileRecording;
		}
	}

	return Drawer;
}

void FDebugDrawer::DrawPoints(const TArrayView<const FVector> Locations, const FLinearColor& Color, const float Thickness, const bool bDepthTest) const
{
	const int32 Num = Locations.Num();

	if (PDI && IsInGameThread())
	{
		for (int32 Idx = 0; Idx < Num; Idx++)
		{
			PDI->DrawPoint(Locations[Idx], Color, Thickness, bDepthTest ? SDPG_World : SDPG_Foreground);
		}
	}

	if (RigVM)
	{
		const FTransform ComponentToWorld = (RigVMComponent && !UE::DrawDebugLibrary::Private::CVarRigVMComponentSpace->GetBool()) ? RigVMComponent->GetComponentTransform() : FTransform::Identity;

		for (int32 Idx = 0; Idx < Num; Idx++)
		{
			RigVM->DrawPoint(FTransform::Identity, ComponentToWorld.InverseTransformPosition(Locations[Idx]), Thickness, Color, bDepthTest ? SDPG_World : SDPG_Foreground);
		}
	}

#if UE_ENABLE_DEBUG_DRAWING
	if (World && IsInGameThread())
	{
		if (Thickness == 0.0f)
		{
			for (int32 Idx = 0; Idx < Num; Idx++)
			{
				::DrawDebugPoint(World, Locations[Idx], 1.0f, Color.ToFColor(true), false, -1.0f, bDepthTest ? 0 : 1);
			}
		}
		else
		{
			for (int32 Idx = 0; Idx < Num; Idx++)
			{
				::DrawDebugLine(World, Locations[Idx], Locations[Idx], Color.ToFColor(true), false, -1.0f, bDepthTest ? 0 : 1, Thickness);
			}
		}
	}

	if (AnimInstance)
	{
		for (int32 Idx = 0; Idx < Num; Idx++)
		{
			AnimInstance->AnimDrawDebugLine(Locations[Idx], Locations[Idx], Color.ToFColor(true), false, -1.0f, Thickness, bDepthTest ? SDPG_World : SDPG_Foreground);
		}
	}

	if (AnimInstanceProxy)
	{
		for (int32 Idx = 0; Idx < Num; Idx++)
		{
			AnimInstanceProxy->AnimDrawDebugLine(Locations[Idx], Locations[Idx], Color.ToFColor(true), false, -1.0f, Thickness, bDepthTest ? SDPG_World : SDPG_Foreground);
		}
	}
#endif

#if ENABLE_VISUAL_LOG
	if (VisualLoggerObject)
	{
		FVisualLogEntry* CurrentEntry = nullptr;
		if (FVisualLogger::CheckVisualLogInputInternal(VisualLoggerObject, VisualLoggerCategory, VisualLoggerVerbosity, /*OutWorld=*/ nullptr, &CurrentEntry))
		{
			if (Thickness == 0.0f)
			{
				FVisualLogShapeElement Element(TEXT(""), Color.ToFColor(false), 0, VisualLoggerCategory);
				Element.Type = EVisualLoggerShapeElement::SinglePoint;
				Element.Verbosity = VisualLoggerVerbosity;
				Element.Points.Reserve(Num);
				for (int32 Idx = 0; Idx < Num; Idx++)
				{
					Element.Points.Add(Locations[Idx]);
				}
				CurrentEntry->ElementsToDraw.Add(Element);
			}
			else
			{
				const uint16 IntegerThickness = Thickness == 0.0f ? 0 : FMath::CeilToInt(FMath::Max(2.0f * Thickness, 0.0f));
				FVisualLogShapeElement Element(TEXT(""), Color.ToFColor(false), IntegerThickness, VisualLoggerCategory);
				Element.Type = EVisualLoggerShapeElement::Segment;
				Element.Verbosity = VisualLoggerVerbosity;
				Element.Points.Reserve(2 * Num);
				for (int32 Idx = 0; Idx < Num; Idx++)
				{
					Element.Points.Add(Locations[Idx]);
					Element.Points.Add(Locations[Idx]);
				}
				CurrentEntry->ElementsToDraw.Add(Element);
			}
		}
		
		const UWorld* VisualLoggerWorld = VisualLoggerObject->GetWorld();

		if (VisualLoggerWorld && IsInGameThread() && bVisualLoggerDrawToScene && (bVisualLoggerDrawToSceneWhileRecording || !FVisualLogger::IsRecording()))
		{
			if (Thickness == 0.0f)
			{
				for (int32 Idx = 0; Idx < Num; Idx++)
				{
					::DrawDebugPoint(VisualLoggerWorld, Locations[Idx], 1.0f, Color.ToFColor(true), false, -1.0f, bDepthTest ? 0 : 1);
				}
			}
			else
			{
				for (int32 Idx = 0; Idx < Num; Idx++)
				{
					::DrawDebugLine(VisualLoggerWorld, Locations[Idx], Locations[Idx], Color.ToFColor(true), false, -1.0f, bDepthTest ? 0 : 1, Thickness);
				}
			}
		}
	}
#endif

	if (Canvas && IsInGameThread())
	{
		UE::DrawDebugLibrary::Private::DrawPointsToCanvas(Canvas, CanvasScale, Locations, Color, Thickness);
	}

	if (CanvasBuffer && IsInGameThread())
	{
		CanvasBuffer->PrimitiveTypes.Add(0);
		CanvasBuffer->PrimitiveNums.Add(Num);
		CanvasBuffer->Colors.Add(Color);
		CanvasBuffer->Thicknesses.Add(Thickness);
		CanvasBuffer->Points.Append(Locations);
	}
}

void FDebugDrawer::DrawLines(const TArrayView<const TPair<FVector, FVector>> Segments, const FLinearColor& Color, const float Thickness, const bool bDepthTest) const
{
	const int32 Num = Segments.Num();

	if (PDI && IsInGameThread())
	{
		if (bDepthTest)
		{
			for (int32 Idx = 0; Idx < Num; Idx++)
			{
				PDI->DrawTranslucentLine(Segments[Idx].Key, Segments[Idx].Value, Color, SDPG_World, FMath::Max(Thickness, 0.0f), 0.0f, Thickness <= 0.0f);
			}
		}
		else
		{
			for (int32 Idx = 0; Idx < Num; Idx++)
			{
				PDI->DrawTranslucentLine(Segments[Idx].Key, Segments[Idx].Value, Color, Thickness == 0.0f ? SDPG_Foreground : SDPG_World, Thickness, Thickness == 0.0f ? 1.0f : UE_MAX_FLT);
			}
		}
	}

	if (RigVM)
	{
		const FTransform ComponentToWorld = (RigVMComponent && !UE::DrawDebugLibrary::Private::CVarRigVMComponentSpace->GetBool()) ? RigVMComponent->GetComponentToWorld() : FTransform::Identity;

		for (int32 Idx = 0; Idx < Num; Idx++)
		{
			RigVM->DrawLine(FTransform::Identity, 
				ComponentToWorld.InverseTransformPosition(Segments[Idx].Key),
				ComponentToWorld.InverseTransformPosition(Segments[Idx].Value), Color, FMath::Max(Thickness, 0.0f), bDepthTest ? SDPG_World : SDPG_Foreground);
		}
	}

#if UE_ENABLE_DEBUG_DRAWING
	if (World && IsInGameThread())
	{
		for (int32 Idx = 0; Idx < Num; Idx++)
		{
			::DrawDebugLine(World, Segments[Idx].Key, Segments[Idx].Value, Color.ToFColor(true), false, -1.0f, bDepthTest ? 0 : 1, Thickness);
		}
	}

	if (AnimInstance)
	{
		for (int32 Idx = 0; Idx < Num; Idx++)
		{
			AnimInstance->AnimDrawDebugLine(Segments[Idx].Key, Segments[Idx].Value, Color.ToFColor(true), false, -1.0f, Thickness, bDepthTest ? SDPG_World : SDPG_Foreground);
		}
	}

	if (AnimInstanceProxy)
	{
		for (int32 Idx = 0; Idx < Num; Idx++)
		{
			AnimInstanceProxy->AnimDrawDebugLine(Segments[Idx].Key, Segments[Idx].Value, Color.ToFColor(true), false, -1.0f, Thickness, bDepthTest ? SDPG_World : SDPG_Foreground);
		}
	}
#endif

#if ENABLE_VISUAL_LOG
	if (VisualLoggerObject)
	{
		FVisualLogEntry* CurrentEntry = nullptr;
		if (FVisualLogger::CheckVisualLogInputInternal(VisualLoggerObject, VisualLoggerCategory, VisualLoggerVerbosity, /*OutWorld=*/ nullptr, &CurrentEntry))
		{
			const uint16 IntegerThickness = Thickness == 0.0f ? 0 : FMath::CeilToInt(FMath::Max(2.0f * Thickness, 0.0f));
			FVisualLogShapeElement Element(TEXT(""), Color.ToFColor(false), IntegerThickness, VisualLoggerCategory);
			Element.Type = EVisualLoggerShapeElement::Segment;
			Element.Verbosity = VisualLoggerVerbosity;
			Element.Points.Reserve(2 * Num);
			for (int32 Idx = 0; Idx < Num; Idx++)
			{
				Element.Points.Add(Segments[Idx].Key);
				Element.Points.Add(Segments[Idx].Value);
			}
			CurrentEntry->ElementsToDraw.Add(Element);
		}

		const UWorld* VisualLoggerWorld = VisualLoggerObject->GetWorld();

		if (VisualLoggerWorld && IsInGameThread() && bVisualLoggerDrawToScene && (bVisualLoggerDrawToSceneWhileRecording || !FVisualLogger::IsRecording()))
		{
			for (int32 Idx = 0; Idx < Num; Idx++)
			{
				::DrawDebugLine(VisualLoggerWorld, Segments[Idx].Key, Segments[Idx].Value, Color.ToFColor(true), false, -1.0f, bDepthTest ? 0 : 1, Thickness);
			}
		}
	}
#endif

	if (Canvas && IsInGameThread())
	{
		UE::DrawDebugLibrary::Private::DrawLinesToCanvas(Canvas, CanvasScale, Segments, Color, Thickness);
	}

	if (CanvasBuffer && IsInGameThread())
	{
		CanvasBuffer->PrimitiveTypes.Add(1);
		CanvasBuffer->PrimitiveNums.Add(Num);
		CanvasBuffer->Colors.Add(Color);
		CanvasBuffer->Thicknesses.Add(Thickness);
		CanvasBuffer->LineSegments.Append(Segments);
	}
}

void FDebugDrawer::VisualLoggerLogString(const FStringView String, const FLinearColor& Color) const
{
#if ENABLE_VISUAL_LOG
	if (VisualLoggerObject)
	{
		FVisualLogEntry* CurrentEntry = nullptr;
		if (FVisualLogger::CheckVisualLogInputInternal(VisualLoggerObject, VisualLoggerCategory, VisualLoggerVerbosity, /*OutWorld=*/ nullptr, &CurrentEntry))
		{
			CurrentEntry->LogLines.Add(FVisualLogLine(VisualLoggerCategory, VisualLoggerVerbosity, FString(String)));
		}
	}
#endif
}

void FDebugDrawer::VisualLoggerDrawString(const FStringView String, const FVector& Location, const FLinearColor& Color) const
{
#if ENABLE_VISUAL_LOG
	if (VisualLoggerObject)
	{
		FVisualLogEntry* CurrentEntry = nullptr;
		if (FVisualLogger::CheckVisualLogInputInternal(VisualLoggerObject, VisualLoggerCategory, VisualLoggerVerbosity, /*OutWorld=*/ nullptr, &CurrentEntry))
		{
			FVisualLogShapeElement Element(FString(String), Color.ToFColor(false), 0, VisualLoggerCategory);
			Element.Type = EVisualLoggerShapeElement::SinglePoint;
			Element.Verbosity = VisualLoggerVerbosity;
			Element.Points.Add(Location);
			CurrentEntry->ElementsToDraw.Add(Element);
		}
	}
#endif
}

void FDebugDrawer::CanvasDrawText(const FVector2D& Location, const FText& Text, const FLinearColor& Color, const FDrawDebugCanvasTextSettings& Settings) const
{
	if (Canvas && IsInGameThread())
	{
		UE::DrawDebugLibrary::Private::DrawTextToCanvas(Canvas, Location, Text, Color, Settings);
	}

	if (CanvasBuffer && IsInGameThread())
	{
		CanvasBuffer->PrimitiveTypes.Add(2);
		CanvasBuffer->PrimitiveNums.Add(1);
		CanvasBuffer->Colors.Add(Color);
		CanvasBuffer->Thicknesses.Add(0.0f);
		CanvasBuffer->Texts.Add({ Location, Text });
		CanvasBuffer->TextSettings.Add(Settings);
	}
}

FDebugDrawer UDrawDebugLibrary::MakeNullDebugDrawer()
{
	return FDebugDrawer::MakeDebugDrawer();
}

FDebugDrawer UDrawDebugLibrary::MakeDebugDrawer(UObject* Object)
{
	return FDebugDrawer::MakeDebugDrawer(Object);
}

FDebugDrawer UDrawDebugLibrary::MakeObjectDebugDrawer(UObject* Object)
{
	return FDebugDrawer::MakeDebugDrawer(Object);
}

FDebugDrawer UDrawDebugLibrary::MakeWorldDebugDrawer(UWorld* World)
{
	return FDebugDrawer::MakeDebugDrawer(World);
}

FDebugDrawer UDrawDebugLibrary::MakeAnimInstanceDebugDrawer(UAnimInstance* AnimInstance)
{
	return FDebugDrawer::MakeDebugDrawer(AnimInstance);
}

FDebugDrawer UDrawDebugLibrary::MakeAnimInstanceProxyDebugDrawer(FAnimInstanceProxy* AnimInstanceProxy)
{
	return FDebugDrawer::MakeDebugDrawer(AnimInstanceProxy);
}

FDebugDrawer UDrawDebugLibrary::MakePDIDebugDrawer(FPrimitiveDrawInterface* PrimitiveDrawInterface)
{
	return FDebugDrawer::MakeDebugDrawer(PrimitiveDrawInterface);
}

FDebugDrawer UDrawDebugLibrary::MakeRigVMDebugDrawer(FRigVMDrawInterface* InRigVMDrawInterface, const USceneComponent* InRigVMComponent)
{
	return FDebugDrawer::MakeDebugDrawer(InRigVMDrawInterface, InRigVMComponent);
}

FDebugDrawer UDrawDebugLibrary::MakeCanvasStructDebugDrawer(FCanvas* InCanvas, const float InCanvasScale)
{
	return FDebugDrawer::MakeDebugDrawer(InCanvas, InCanvasScale);
}

FDebugDrawer UDrawDebugLibrary::MakeCanvasDebugDrawer(UCanvas* InCanvas, const float InCanvasScale)
{
	return InCanvas ? FDebugDrawer::MakeDebugDrawer(InCanvas->Canvas, InCanvasScale) : FDebugDrawer::MakeDebugDrawer();
}

FDebugDrawer UDrawDebugLibrary::MakeCanvasDebugDrawerFromBuffer(const TSharedPtr<FDebugDrawerCanvasBuffer>& CanavsBuffer)
{
	return FDebugDrawer::MakeDebugDrawer(CanavsBuffer);
}

FDebugDrawer UDrawDebugLibrary::MakeVisualLoggerDebugDrawer(const UObject* Object, const FName Category, const EDrawDebugLogVerbosity Verbosity, const bool bDrawToScene,const bool bDrawToSceneWhileRecording)
{
	return FDebugDrawer::MakeVisualLoggerDebugDrawer(Object, Category, Verbosity, bDrawToScene, bDrawToSceneWhileRecording);
}

FDebugDrawer UDrawDebugLibrary::MakeVisualLoggerDebugDrawerFromObject(const UObject* Object, const FName Category, const EDrawDebugLogVerbosity Verbosity, const bool bDrawToScene, const bool bDrawToSceneWhileRecording)
{
	return FDebugDrawer::MakeVisualLoggerDebugDrawer(Object, Category, Verbosity, bDrawToScene, bDrawToSceneWhileRecording);
}

FDebugDrawer UDrawDebugLibrary::MakeVisualLoggerDebugDrawerFromObjectWithCategory(const UObject* Object, const FLogCategoryBase& Category, const ELogVerbosity::Type Verbosity, const bool bDrawToScene, const bool bDrawToSceneWhileRecording)
{
	return FDebugDrawer::MakeVisualLoggerDebugDrawer(Object, Category, Verbosity, bDrawToScene, bDrawToSceneWhileRecording);
}

FDebugDrawer UDrawDebugLibrary::MakeScreenDebugDrawer(const float Scale)
{
	return FDebugDrawer::MakeScreenDebugDrawer(Scale);
}

FDebugDrawer UDrawDebugLibrary::MakeMergedDebugDrawer(const TArray<FDebugDrawer>& DebugDrawers)
{
	return FDebugDrawer::MakeMergedDebugDrawer(DebugDrawers);
}

FDebugDrawer UDrawDebugLibrary::MakeMergedDebugDrawerArrayView(const TArrayView<const FDebugDrawer> DebugDrawers)
{
	return FDebugDrawer::MakeMergedDebugDrawer(DebugDrawers);
}

void UDrawDebugLibrary::VisualLoggerLogString(const FDebugDrawer& Drawer, const FString& String, const FLinearColor Color)
{
	Drawer.VisualLoggerLogString(String, Color);
}

void UDrawDebugLibrary::VisualLoggerLogStringView(const FDebugDrawer& Drawer, const FStringView String, const FLinearColor Color)
{
	Drawer.VisualLoggerLogString(String, Color);
}

void UDrawDebugLibrary::VisualLoggerLogText(const FDebugDrawer& Drawer, const FText& Text, const FLinearColor Color)
{
	Drawer.VisualLoggerLogString(Text.ToString(), Color);
}

void UDrawDebugLibrary::DrawDebugCanvasText(const FDebugDrawer& Drawer, const FVector2D& Location, const FText& Text, const FLinearColor Color, const FDrawDebugCanvasTextSettings& Settings)
{
	Drawer.CanvasDrawText(Location, Text, Color, Settings);
}

void UDrawDebugLibrary::DrawDebugCanvasString(const FDebugDrawer& Drawer, const FVector2D& Location, const FString& String, const FLinearColor Color, const FDrawDebugCanvasTextSettings& Settings)
{
	Drawer.CanvasDrawText(Location, FText::FromString(String), Color, Settings);
}

void UDrawDebugLibrary::DrawDebugCanvasStringView(const FDebugDrawer& Drawer, const FVector2D& Location, const FStringView String, const FLinearColor Color, const FDrawDebugCanvasTextSettings& Settings)
{
	Drawer.CanvasDrawText(Location, FText::FromStringView(String), Color, Settings);
}

void UDrawDebugLibrary::DrawDebugCanvasName(const FDebugDrawer& Drawer, const FVector2D& Location, const FName& Name, const FLinearColor Color, const FDrawDebugCanvasTextSettings& Settings)
{
	Drawer.CanvasDrawText(Location, FText::FromName(Name), Color, Settings);
}

void UDrawDebugLibrary::MakeLinearlySpacedFloatArray(TArray<float>& OutValues, const float Start, const float Stop, const int32 Num)
{
	OutValues.SetNumUninitialized(Num);
	MakeLinearlySpacedFloatArrayView(OutValues, Start, Stop);
}

void UDrawDebugLibrary::MakeLinearlySpacedFloatArrayView(TArrayView<float> OutValues, const float Start, const float Stop)
{
	const int32 Num = OutValues.Num();

	if (Num == 0) { return; }

	if (Num == 1)
	{
		OutValues[0] = Start;
		return;
	}

	for (int32 Idx = 0; Idx < Num; Idx++)
	{
		OutValues[Idx] = Start + (Stop - Start) * ((float)(Idx) / (Num - 1));
	}
}

void UDrawDebugLibrary::AddToFloatHistoryArray(TArray<float>& InOutValues, const float NewValue, const int32 MaxHistoryNum)
{
	if (MaxHistoryNum <= 0)
	{
		InOutValues.Empty();
		return;
	}

	const int32 Shift = InOutValues.Num() - MaxHistoryNum + 1;

	if (Shift > 0)
	{
		for (int32 Idx = 0; Idx < InOutValues.Num() - Shift; Idx++)
		{
			InOutValues[Idx] = InOutValues[Idx + Shift];
		}

		InOutValues.SetNum(MaxHistoryNum);
		InOutValues.Last() = NewValue;
	}
	else
	{
		InOutValues.Add(NewValue);
	}
}

void UDrawDebugLibrary::AddToVectorHistoryArray(TArray<FVector>& InOutValues, const FVector NewValue, const int32 MaxHistoryNum)
{
	if (MaxHistoryNum <= 0)
	{
		InOutValues.Empty();
		return;
	}

	const int32 Shift = InOutValues.Num() - MaxHistoryNum + 1;

	if (Shift > 0)
	{
		for (int32 Idx = 0; Idx < InOutValues.Num() - Shift; Idx++)
		{
			InOutValues[Idx] = InOutValues[Idx + Shift];
		}

		InOutValues.SetNum(MaxHistoryNum);
		InOutValues.Last() = NewValue;
	}
	else
	{
		InOutValues.Add(NewValue);
	}
}

void UDrawDebugLibrary::AddToNameHistoryArray(TArray<FName>& InOutValues, const FName NewValue, const int32 MaxHistoryNum)
{
	if (MaxHistoryNum <= 0)
	{
		InOutValues.Empty();
		return;
	}

	const int32 Shift = InOutValues.Num() - MaxHistoryNum + 1;

	if (Shift > 0)
	{
		for (int32 Idx = 0; Idx < InOutValues.Num() - Shift; Idx++)
		{
			InOutValues[Idx] = InOutValues[Idx + Shift];
		}

		InOutValues.SetNum(MaxHistoryNum);
		InOutValues.Last() = NewValue;
	}
	else
	{
		InOutValues.Add(NewValue);
	}
}

void UDrawDebugLibrary::DrawDebugLocalOffset(
	FVector& OutLocation,
	FRotator& OutRotation,
	const FVector DrawLocation,
	const FRotator DrawRotation,
	const FVector DrawOffset)
{
	OutLocation = DrawLocation + DrawRotation.RotateVector(DrawOffset);
	OutRotation = DrawRotation;
}

void UDrawDebugLibrary::DrawDebugOrientUprightToFloor(
	FVector& OutLocation,
	FRotator& OutRotation,
	const FVector DrawWorldLocation,
	const FVector DrawUprightOffset)
{
	const FQuat Reorientation = FQuat::MakeFromRotationVector(FVector(0.0f, +UE_PI / 2, 0.0f));
	OutLocation = DrawWorldLocation + Reorientation.RotateVector(DrawUprightOffset);
	OutRotation = Reorientation.Rotator();
}

void UDrawDebugLibrary::DrawDebugOrientFloorToUpright(
	FVector& OutLocation,
	FRotator& OutRotation,
	const FVector DrawWorldLocation,
	const FVector DrawFloorOffset)
{
	const FQuat Reorientation = FQuat::MakeFromRotationVector(FVector(0.0f, -UE_PI / 2, 0.0f));
	OutLocation = DrawWorldLocation + Reorientation.RotateVector(DrawFloorOffset);
	OutRotation = Reorientation.Rotator();
}

void UDrawDebugLibrary::DrawDebugOrientUprightToCamera(
	FVector& OutLocation,
	FRotator& OutRotation,
	const FVector DrawWorldLocation,
	const FVector DrawUprightOffset,
	const FRotator CameraRotation,
	const bool bUseOnlyYaw)
{
	const FRotator ActualCameraRotation = bUseOnlyYaw ? FRotator(0.0f, CameraRotation.Yaw, 0.0f) : CameraRotation;

	OutLocation = DrawWorldLocation + ActualCameraRotation.RotateVector(DrawUprightOffset);
	OutRotation = ActualCameraRotation;
}

void UDrawDebugLibrary::DrawDebugOrientFloorToCamera(
	FVector& OutLocation,
	FRotator& OutRotation,
	const FVector DrawWorldLocation,
	const FVector DrawFloorOffset,
	const FRotator CameraRotation,
	const bool bUseOnlyYaw)
{
	const FQuat Reorientation = FQuat::MakeFromRotationVector(FVector(0.0f, -UE_PI / 2, 0.0f));
	const FRotator ActualCameraRotation = bUseOnlyYaw ? FRotator(0.0f, CameraRotation.Yaw, 0.0f) : CameraRotation;
	const FQuat FinalRotation = ActualCameraRotation.Quaternion() * Reorientation;

	OutLocation = DrawWorldLocation + FinalRotation.RotateVector(DrawFloorOffset);
	OutRotation = FinalRotation.Rotator();
}

void UDrawDebugLibrary::DrawDebugOrientUprightToCanvas(
	FVector& OutLocation,
	FRotator& OutRotation,
	const FVector2D DrawCanvasLocation)
{
	const FQuat Reorientation = FQuat::MakeFromRotationVector(FVector(+UE_PI / 2, 0.0f, 0.0f)) * FQuat::MakeFromRotationVector(FVector(0.0f, 0.0f, -UE_PI / 2));
	OutLocation = FVector(DrawCanvasLocation.X, DrawCanvasLocation.Y, 0.0f);
	OutRotation = Reorientation.Rotator();
}

FDrawDebugPointStyle::FDrawDebugPointStyle(
	const FLinearColor InColor,
	const float InThickness) 
	: Color(InColor)
	, Thickness(InThickness) 
{}

FDrawDebugLineStyle::FDrawDebugLineStyle(
	const FLinearColor InColor,
	const float InThickness,
	const EDrawDebugLineType InLineType,
	const float InDashWidth,
	const float InDashSpacing,
	const float InDotSpacing)
	: Color(InColor)
	, Thickness(InThickness)
	, LineType(InLineType)
	, DashWidth(InDashWidth)
	, DashSpacing(InDashSpacing)
	, DotSpacing(InDotSpacing)
{}

FDrawDebugPointStyle UDrawDebugLibrary::MakeDrawDebugPointStyleFromColor(const FLinearColor& Color)
{
	FDrawDebugPointStyle Out;
	Out.Color = Color;
	return Out;
}

FDrawDebugPointStyle UDrawDebugLibrary::MakeDrawDebugPointStyleFromColorAndThickness(const FLinearColor Color, const float Thickness)
{
	FDrawDebugPointStyle Out;
	Out.Color = Color;
	Out.Thickness = Thickness;
	return Out;
}

FDrawDebugLineStyle UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(const FLinearColor& Color)
{
	FDrawDebugLineStyle Out;
	Out.Color = Color;
	return Out;
}

FDrawDebugLineStyle UDrawDebugLibrary::MakeDrawDebugLineStyleFromColorAndThickness(const FLinearColor Color, const float Thickness)
{
	FDrawDebugLineStyle Out;
	Out.Color = Color;
	Out.Thickness = Thickness;
	return Out;
}

FDrawDebugPointStyle UDrawDebugLibrary::DrawDebugPointStyleFromLineStyle(const FDrawDebugLineStyle& LineStyle)
{
	FDrawDebugPointStyle Out;
	Out.Color = LineStyle.Color;
	Out.Thickness = LineStyle.Thickness;
	return Out;
}

FDrawDebugLineStyle UDrawDebugLibrary::DrawDebugLineStyleFromPointStyle(const FDrawDebugPointStyle& PointStyle)
{
	FDrawDebugLineStyle Out;
	Out.Color = PointStyle.Color;
	Out.Thickness = PointStyle.Thickness;
	return Out;
}

FLinearColor UDrawDebugLibrary::DrawDebugPointStyleColor(const FDrawDebugPointStyle& PointStyle)
{
	return PointStyle.Color;
}

FLinearColor UDrawDebugLibrary::DrawDebugLineStyleColor(const FDrawDebugLineStyle& LineStyle)
{
	return LineStyle.Color;
}

FDrawDebugLineStyle UDrawDebugLibrary::DrawDebugLineStyleWithColor(const FDrawDebugLineStyle& LineStyle, const FLinearColor Color)
{
	FDrawDebugLineStyle Out = LineStyle;
	Out.Color = Color;
	return Out;
}

FDrawDebugLineStyle UDrawDebugLibrary::DrawDebugLineStyleWithColorNoOpacity(const FDrawDebugLineStyle& LineStyle, const FLinearColor Color)
{
	FDrawDebugLineStyle Out = LineStyle;
	Out.Color.R = Color.R;
	Out.Color.G = Color.G;
	Out.Color.B = Color.B;
	return Out;
}

FDrawDebugLineStyle UDrawDebugLibrary::DrawDebugLineStyleWithThickness(const FDrawDebugLineStyle& LineStyle, const float Thickness)
{
	FDrawDebugLineStyle Out = LineStyle;
	Out.Thickness = Thickness;
	return Out;
}

FDrawDebugLineStyle UDrawDebugLibrary::DrawDebugLineStyleWithType(const FDrawDebugLineStyle& LineStyle, const EDrawDebugLineType LineType)
{
	FDrawDebugLineStyle Out = LineStyle;
	Out.LineType = LineType;
	return Out;
}

void UDrawDebugLibrary::DrawDebugPoint(const FDebugDrawer& Drawer, const FVector& Location, const FDrawDebugPointStyle& PointStyle, const bool bDepthTest)
{
	DrawDebugPointsArrayView(Drawer, { Location }, PointStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugPoints(const FDebugDrawer& Drawer, const TArray<FVector>& Locations, const FDrawDebugPointStyle& PointStyle, const bool bDepthTest)
{
	DrawDebugPointsArrayView(Drawer, Locations, PointStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugPointsArrayView(const FDebugDrawer& Drawer, const TArrayView<const FVector> Locations, const FDrawDebugPointStyle& PointStyle, const bool bDepthTest)
{
	Drawer.DrawPoints(Locations, PointStyle.Color, PointStyle.Thickness, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugLine(const FDebugDrawer& Drawer, const FVector& StartLocation, const FVector& EndLocation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest)
{
	DrawDebugLinesArrayView(Drawer, { StartLocation }, { EndLocation }, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugLines(const FDebugDrawer& Drawer, const TArray<FVector>& StartLocations, const TArray<FVector>& EndLocations, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest)
{
	DrawDebugLinesArrayView(Drawer, StartLocations, EndLocations, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugLinesArrayView(const FDebugDrawer& Drawer, const TArrayView<const FVector> StartLocations, const TArrayView<const FVector> EndLocations, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest)
{
	TArray<TPair<FVector, FVector>, TInlineAllocator<64>> Segments;
	const int32 Num = FMath::Min(StartLocations.Num(), EndLocations.Num());
	Segments.SetNumUninitialized(Num);
	for (int32 Idx = 0; Idx < Num; Idx++)
	{
		Segments[Idx] = { StartLocations[Idx], EndLocations[Idx] };
	}

	DrawDebugLinesPairsArrayView(Drawer, Segments, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugLinesPairsArrayView(const FDebugDrawer& Drawer, const TArrayView<const TPair<FVector, FVector>> Segments, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest)
{
	const float SafeThickness = FMath::Max(LineStyle.Thickness + 1.0f, 0.0f);

	switch (LineStyle.LineType)
	{

	case EDrawDebugLineType::Solid:
	{
		Drawer.DrawLines(Segments, LineStyle.Color, LineStyle.Thickness, bDepthTest);
		return;
	}

	case EDrawDebugLineType::Dotted:
	{
		const float SafeDotSpacing = FMath::Max(LineStyle.DotSpacing, 1.0f);
		TArray<FVector> Dots;

		float Dist = 0.0f;
		const int32 Num = Segments.Num();
		for (int32 Idx = 0; Idx < Num; Idx++)
		{
			const FVector Start = Segments[Idx].Key;
			const FVector End = Segments[Idx].Value;
			const FVector Direction = (End - Start).GetSafeNormal();
			const float Length = FMath::Max((End - Start).Length(), UE_SMALL_NUMBER);
			for (; Dist < Length; Dist += SafeDotSpacing * SafeThickness)
			{
				Dots.Add(Start + Direction * Dist);
			}
			Dist = FMath::Fmod(Dist, Length);
		}

		Drawer.DrawPoints(Dots, LineStyle.Color, LineStyle.Thickness, bDepthTest);
		return;
	}

	case EDrawDebugLineType::Dashed:
	{
		const float SafeDashSpacing = FMath::Max(LineStyle.DashSpacing, 1.0f);
		TArray<TPair<FVector, FVector>, TInlineAllocator<64>> Dashes;

		float Dist = 0.0f;
		const int32 Num = Segments.Num();
		for (int32 Idx = 0; Idx < Num; Idx++)
		{
			const FVector Start = Segments[Idx].Key;
			const FVector End = Segments[Idx].Value;
			const FVector Direction = (End - Start).GetSafeNormal();
			const float Length = FMath::Max((End - Start).Length(), UE_SMALL_NUMBER);
			for (; Dist < Length; Dist += LineStyle.DashWidth + SafeDashSpacing * SafeThickness)
			{
				Dashes.Add({ 
					Start + Direction * FMath::Clamp(Dist, 0.0f, Length), 
					Start + Direction * FMath::Clamp(Dist + LineStyle.DashWidth, 0.0f, Length) });
			}
			Dist = FMath::Fmod(Dist, Length);
		}

		Drawer.DrawLines(Dashes, LineStyle.Color, LineStyle.Thickness, bDepthTest);
		return;
	}

	default: checkNoEntry();
	}
}


void UDrawDebugLibrary::DrawDebugTriangularBasePyramid(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Length, const float Width)
{
	const FVector V0 = Location + Rotation.RotateVector(FVector(0, 0, Length/2));
	const FVector V1 = Location + Rotation.RotateVector(FVector(-Width / 4 * FMath::Sqrt(3.0f), -Width / 2, -Length / 2));
	const FVector V2 = Location + Rotation.RotateVector(FVector(-Width / 4 * FMath::Sqrt(3.0f), +Width / 2, -Length / 2));
	const FVector V3 = Location + Rotation.RotateVector(FVector(+Width / 4 * FMath::Sqrt(3.0f), 0.0f, -Length / 2));

	DrawDebugLinesPairsArrayView(Drawer, {
		{ V0, V1 },
		{ V0, V2 },
		{ V0, V3 },
		{ V1, V2 },
		{ V2, V3 },
		{ V3, V1 },
	}, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugSquareBasePyramid(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Length, const float Width)
{
	const FVector V0 = Location + Rotation.RotateVector(FVector(0, 0, Length / 2));
	const FVector V1 = Location + Rotation.RotateVector(FVector(-Width / 2, -Width / 2, -Length / 2));
	const FVector V2 = Location + Rotation.RotateVector(FVector(-Width / 2, +Width / 2, -Length / 2));
	const FVector V3 = Location + Rotation.RotateVector(FVector(+Width / 2, +Width / 2, -Length / 2));
	const FVector V4 = Location + Rotation.RotateVector(FVector(+Width / 2, -Width / 2, -Length / 2));

	DrawDebugLinesPairsArrayView(Drawer, {
		{ V0, V1 },
		{ V0, V2 },
		{ V0, V3 },
		{ V0, V4 },
		{ V1, V2 },
		{ V2, V3 },
		{ V3, V4 },
		{ V4, V1 },
		}, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugCone(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Length, const float Radius, const int32 Segments)
{
	const int32 Num = FMath::Max(Segments, 2);

	const FVector Peak = Location + Rotation.RotateVector(FVector(0, 0, Length / 2));

	TArray<TPair<FVector, FVector>, TInlineAllocator<9 * 2>> LineSegments;
	LineSegments.Reserve(Num * 2);

	for (int32 Idx = 0; Idx < Num; Idx++)
	{
		const FVector V0 = FVector(Radius * FMath::Sin(UE_TWO_PI * (float)(Idx + 0) / Num), Radius * FMath::Cos(UE_TWO_PI * (float)(Idx + 0) / Num), -Length / 2);
		const FVector V1 = FVector(Radius * FMath::Sin(UE_TWO_PI * (float)(Idx + 1) / Num), Radius * FMath::Cos(UE_TWO_PI * (float)(Idx + 1) / Num), -Length / 2);
		LineSegments.Add({ Peak, Location + Rotation.RotateVector(V0) });
		LineSegments.Add({ Location + Rotation.RotateVector(V0), Location + Rotation.RotateVector(V1) });
	}

	DrawDebugLinesPairsArrayView(Drawer, LineSegments, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugConeLookAt(const FDebugDrawer& Drawer, const FVector& Location, const FVector& Direction, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Length, const float Angle, const int32 Segments)
{
	const FRotator ConeRotation = (Direction.Rotation().Quaternion() * FQuat::MakeFromRotationVector(FVector(0.0f, -UE_PI / 2.0f, 0.0f))).Rotator();
	const FVector ConeLocation = Location + Direction.GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector) * Length / 2.0f;
	const float ConeRadius = Length * FMath::Tan(FMath::DegreesToRadians(Angle));
	DrawDebugCone(Drawer, ConeLocation, ConeRotation, LineStyle, bDepthTest, Length, ConeRadius, Segments);
}

void UDrawDebugLibrary::DrawDebugArc(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const float Angle, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius, const int32 Segments)
{
	TArray<TPair<FVector, FVector>, TInlineAllocator<17>> LineSegments;
	LineSegments.Reserve(Segments);

	for (int32 SegmentIdx = 0; SegmentIdx < Segments; SegmentIdx++)
	{
		const float Angle0 = (((float)SegmentIdx + 0) / Segments) * FMath::DegreesToRadians(Angle);
		const float Angle1 = (((float)SegmentIdx + 1) / Segments) * FMath::DegreesToRadians(Angle);
		const FVector Location0 = Location + Rotation.RotateVector(Radius * FVector(FMath::Cos(Angle0), FMath::Sin(Angle0), 0.0f));
		const FVector Location1 = Location + Rotation.RotateVector(Radius * FVector(FMath::Cos(Angle1), FMath::Sin(Angle1), 0.0f));
		LineSegments.Add({ Location0, Location1 });
	}

	DrawDebugLinesPairsArrayView(Drawer, LineSegments, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugCircle(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius, const int32 Segments)
{
	DrawDebugArc(Drawer, Location, Rotation, 360.0f, LineStyle, bDepthTest, Radius, FMath::Max(Segments, 3));
}

void UDrawDebugLibrary::DrawDebugCircleTick(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const float Angle, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius, const float Length, bool bInside)
{
	DrawDebugCircleTicksArrayView(Drawer, Location, Rotation, { Angle }, LineStyle, bDepthTest, Radius, Length, bInside);
}

void UDrawDebugLibrary::DrawDebugCircleTicks(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const TArray<float>& Angles, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius, const float Length, bool bInside)
{
	DrawDebugCircleTicksArrayView(Drawer, Location, Rotation, Angles, LineStyle, bDepthTest, Radius, Length, bInside);
}

void UDrawDebugLibrary::DrawDebugCircleTicksArrayView(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const TArrayView<const float> Angles, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius, const float Length, bool bInside)
{
	const int32 TickNum = Angles.Num();

	TArray<TPair<FVector, FVector>, TInlineAllocator<17>> LineSegments;
	LineSegments.Reserve(TickNum);

	for (int32 TickIdx = 0; TickIdx < TickNum; TickIdx++)
	{
		const float Angle = Angles[TickIdx];
		const float Offset = bInside ? -Length : +Length;
		const FVector Start = Location + Rotation.RotateVector(Radius * FVector(FMath::Cos(FMath::DegreesToRadians(Angle)), FMath::Sin(FMath::DegreesToRadians(Angle)), 0.0f));
		const FVector End = Location + Rotation.RotateVector((Radius + Offset) * FVector(FMath::Cos(FMath::DegreesToRadians(Angle)), FMath::Sin(FMath::DegreesToRadians(Angle)), 0.0f));
		LineSegments.Add({ Start, End });
	}

	DrawDebugLinesPairsArrayView(Drawer, LineSegments, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugTriangle(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius)
{
	DrawDebugCircle(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius, 3);
}

void UDrawDebugLibrary::DrawDebugSquare(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float HalfLength)
{
	const FVector V0 = Location + Rotation.RotateVector(HalfLength * FVector(+1, +1, 0));
	const FVector V1 = Location + Rotation.RotateVector(HalfLength * FVector(-1, +1, 0));
	const FVector V2 = Location + Rotation.RotateVector(HalfLength * FVector(-1, -1, 0));
	const FVector V3 = Location + Rotation.RotateVector(HalfLength * FVector(+1, -1, 0));

	DrawDebugLinesPairsArrayView(Drawer, {
		{ V0, V1 },
		{ V1, V2 },
		{ V2, V3 },
		{ V3, V0 },
		}, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugCross(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float HalfLength)
{
	const FVector V0 = Location + Rotation.RotateVector(HalfLength * FVector(+1, +1, 0));
	const FVector V1 = Location + Rotation.RotateVector(HalfLength * FVector(-1, +1, 0));
	const FVector V2 = Location + Rotation.RotateVector(HalfLength * FVector(-1, -1, 0));
	const FVector V3 = Location + Rotation.RotateVector(HalfLength * FVector(+1, -1, 0));

	DrawDebugLinesPairsArrayView(Drawer, {
		{ V0, V2 },
		{ V1, V3 },
		}, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugDiamond(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius)
{
	DrawDebugCircle(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius, 4);
}

void UDrawDebugLibrary::DrawDebugPentagon(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius)
{
	DrawDebugCircle(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius, 5);
}

void UDrawDebugLibrary::DrawDebugHexagon(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius)
{
	DrawDebugCircle(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius, 6);
}

void UDrawDebugLibrary::DrawDebugHeptagon(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius)
{
	DrawDebugCircle(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius, 7);
}

void UDrawDebugLibrary::DrawDebugOctagon(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius)
{
	DrawDebugCircle(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius, 8);
}

void UDrawDebugLibrary::DrawDebugNonagon(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius)
{
	DrawDebugCircle(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius, 9);
}

void UDrawDebugLibrary::DrawDebugDecagon(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius)
{
	DrawDebugCircle(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius, 10);
}

void UDrawDebugLibrary::DrawDebugRegularPolygon(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius, const int32 Sides)
{
	DrawDebugCircle(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius, Sides);
}

void UDrawDebugLibrary::DrawDebugCheckMark(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius)
{
	const FVector P0 = Location + Rotation.RotateVector(Radius * FVector(0.0f, -0.2f, 0.0f));
	const FVector P1 = Location + Rotation.RotateVector(Radius * FVector(-0.2f, 0.0f,0.0f));
	const FVector P2 = Location + Rotation.RotateVector(Radius * FVector(0.8f, 0.6f, 0.0f));

	DrawDebugLinesPairsArrayView(Drawer,
		{
			{ P0, P1 },
			{ P1, P2 },
		}, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugCheckBox(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float HalfLength, const bool bChecked, const float TickRatio)
{
	DrawDebugSquare(Drawer, Location, Rotation, LineStyle, bDepthTest, HalfLength);
	if (bChecked)
	{
		DrawDebugCheckMark(Drawer, Location, Rotation, LineStyle, bDepthTest, HalfLength * TickRatio);
	}
}

void UDrawDebugLibrary::DrawDebugCrossBox(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float HalfLength, const bool bCrossed, const float CrossRatio)
{
	DrawDebugSquare(Drawer, Location, Rotation, LineStyle, bDepthTest, HalfLength);
	if (bCrossed)
	{
		DrawDebugCross(Drawer, Location, Rotation, LineStyle, bDepthTest, HalfLength * CrossRatio);
	}
}

void UDrawDebugLibrary::DrawDebugRadioButton(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius, const bool bSelected, const float InnerButtonRatio, const int32 Segments)
{
	DrawDebugCircle(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius, Segments);
	if (bSelected)
	{
		DrawDebugCircle(Drawer, Location, Rotation, LineStyle, bDepthTest, InnerButtonRatio * Radius, Segments);
	}
}

void UDrawDebugLibrary::DrawDebugCircleOutline(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float InnerRadius, const float OuterRadius, const int32 Segments)
{
	DrawDebugCircle(Drawer, Location, Rotation, LineStyle, bDepthTest, InnerRadius, Segments);
	DrawDebugCircle(Drawer, Location, Rotation, LineStyle, bDepthTest, OuterRadius, Segments);
}

void UDrawDebugLibrary::DrawDebugLocator(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float DrawRadius)
{

	DrawDebugLinesPairsArrayView(Drawer, {
		{ Location - Rotation.RotateVector(FVector(DrawRadius, 0, 0)), Location + Rotation.RotateVector(FVector(DrawRadius, 0, 0)) },
		{ Location - Rotation.RotateVector(FVector(0, DrawRadius, 0)), Location + Rotation.RotateVector(FVector(0, DrawRadius, 0)) },
		{ Location - Rotation.RotateVector(FVector(0, 0, DrawRadius)), Location + Rotation.RotateVector(FVector(0, 0, DrawRadius)) },
		}, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugCrossLocator(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float DrawRadius)
{
	DrawDebugLinesPairsArrayView(Drawer, {
		{ Location + Rotation.RotateVector(FVector(+DrawRadius, +DrawRadius, -DrawRadius)), Location + Rotation.RotateVector(FVector(-DrawRadius, -DrawRadius, +DrawRadius)) },
		{ Location + Rotation.RotateVector(FVector(-DrawRadius, +DrawRadius, -DrawRadius)), Location + Rotation.RotateVector(FVector(+DrawRadius, -DrawRadius, +DrawRadius)) },
		{ Location + Rotation.RotateVector(FVector(+DrawRadius, -DrawRadius, -DrawRadius)), Location + Rotation.RotateVector(FVector(-DrawRadius, +DrawRadius, +DrawRadius)) },
		{ Location + Rotation.RotateVector(FVector(-DrawRadius, -DrawRadius, -DrawRadius)), Location + Rotation.RotateVector(FVector(+DrawRadius, +DrawRadius, +DrawRadius)) },
		}, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugBox(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const FVector HalfExtents)
{
	const FVector C0 = Location + Rotation.RotateVector(FVector(+1, +1, -1) * HalfExtents);
	const FVector C1 = Location + Rotation.RotateVector(FVector(+1, -1, -1) * HalfExtents);
	const FVector C2 = Location + Rotation.RotateVector(FVector(-1, -1, -1) * HalfExtents);
	const FVector C3 = Location + Rotation.RotateVector(FVector(-1, +1, -1) * HalfExtents);
	const FVector C4 = Location + Rotation.RotateVector(FVector(+1, +1, +1) * HalfExtents);
	const FVector C5 = Location + Rotation.RotateVector(FVector(+1, -1, +1) * HalfExtents);
	const FVector C6 = Location + Rotation.RotateVector(FVector(-1, -1, +1) * HalfExtents);
	const FVector C7 = Location + Rotation.RotateVector(FVector(-1, +1, +1) * HalfExtents);

	DrawDebugLinesPairsArrayView(Drawer, {
		{ C0, C1 },
		{ C1, C2 },
		{ C2, C3 },
		{ C3, C0 },

		{ C4, C5 },
		{ C5, C6 },
		{ C6, C7 },
		{ C7, C4 },

		{ C0, C4 },
		{ C1, C5 },
		{ C2, C6 },
		{ C3, C7 },
		}, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugBoxStruct(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const FBox& Box)
{
	FVector V[8];
	Box.GetVertices(V);

	for (int32 Idx = 0; Idx < 8; Idx++)
	{
		V[Idx] = Location + Rotation.RotateVector(V[Idx]);
	}

	DrawDebugLinesPairsArrayView(Drawer, {
		{ V[4], V[3] },
		{ V[3], V[0] },
		{ V[0], V[2] },
		{ V[2], V[4] },

		{ V[7], V[5] },
		{ V[5], V[1] },
		{ V[1], V[6] },
		{ V[6], V[7] },

		{ V[4], V[7] },
		{ V[3], V[5] },
		{ V[0], V[1] },
		{ V[2], V[6] },
		}, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugBoxSphereBounds(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const FBoxSphereBounds& Bounds)
{
	DrawDebugBoxStruct(Drawer, Location, Rotation, LineStyle, bDepthTest, FBox::BuildAABB(Bounds.Origin, Bounds.BoxExtent));
}

void UDrawDebugLibrary::DrawDebugSphere(
	const FDebugDrawer& Drawer,
	const FVector& Location,
	const FRotator& Rotation,
	const FDrawDebugLineStyle& LineStyle,
	const bool bDepthTest,
	const float Radius,
	const int32 Segments)
{
	const int32 SegmentNum = (FMath::Max(Segments, 4) / 2) * 2;

	TArray<TPair<FVector, FVector>, TInlineAllocator<8 * 8>> LineSegments;
	LineSegments.Reserve(SegmentNum * SegmentNum);

	for (int32 X = 0; X < SegmentNum; X++)
	{
		const float XA0 = ((float)(X + 0) / SegmentNum) * UE_TWO_PI;
		const float XA1 = ((float)(X + 1) / SegmentNum) * UE_TWO_PI;

		for (int32 Y = 0; Y < SegmentNum / 2; Y++)
		{
			const float YA0 = ((float)(Y + 0) / SegmentNum) * UE_TWO_PI;
			const float YA1 = ((float)(Y + 1) / SegmentNum) * UE_TWO_PI;

			const FVector V0 = Location + Rotation.RotateVector(FVector(FMath::Cos(XA0) * FMath::Sin(YA0), FMath::Sin(XA0) * FMath::Sin(YA0), FMath::Cos(YA0)) * Radius);
			const FVector V1 = Location + Rotation.RotateVector(FVector(FMath::Cos(XA1) * FMath::Sin(YA0), FMath::Sin(XA1) * FMath::Sin(YA0), FMath::Cos(YA0)) * Radius);
			const FVector V2 = Location + Rotation.RotateVector(FVector(FMath::Cos(XA0) * FMath::Sin(YA1), FMath::Sin(XA0) * FMath::Sin(YA1), FMath::Cos(YA1)) * Radius);

			LineSegments.Add({ V0, V1 });
			LineSegments.Add({ V0, V2 });
		}
	}

	DrawDebugLinesPairsArrayView(Drawer, LineSegments, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugSimpleSphere(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius, const int32 Segments)
{
	TArray<TPair<FVector, FVector>, TInlineAllocator<39>> LineSegments;
	LineSegments.Reserve(Segments*3);

	for (int32 SegmentIdx = 0; SegmentIdx < Segments; SegmentIdx++)
	{
		const float Angle0 = (((float)SegmentIdx + 0) / Segments) * UE_TWO_PI;
		const float Angle1 = (((float)SegmentIdx + 1) / Segments) * UE_TWO_PI;
		const FVector X0 = Location + Rotation.RotateVector(Radius * FVector(0.0f, FMath::Cos(Angle0), FMath::Sin(Angle0)));
		const FVector X1 = Location + Rotation.RotateVector(Radius * FVector(0.0f, FMath::Cos(Angle1), FMath::Sin(Angle1)));
		const FVector Y0 = Location + Rotation.RotateVector(Radius * FVector(FMath::Cos(Angle0), 0.0f, FMath::Sin(Angle0)));
		const FVector Y1 = Location + Rotation.RotateVector(Radius * FVector(FMath::Cos(Angle1), 0.0f, FMath::Sin(Angle1)));
		const FVector Z0 = Location + Rotation.RotateVector(Radius * FVector(FMath::Cos(Angle0), FMath::Sin(Angle0), 0.0f));
		const FVector Z1 = Location + Rotation.RotateVector(Radius * FVector(FMath::Cos(Angle1), FMath::Sin(Angle1), 0.0f));
		LineSegments.Add({ X0, X1 });
		LineSegments.Add({ Y0, Y1 });
		LineSegments.Add({ Z0, Z1 });
	}

	DrawDebugLinesPairsArrayView(Drawer, LineSegments, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugCapsule(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius, const float HalfLength, const int32 Segments)
{
	const int32 SegmentNum = (FMath::Max(Segments, 4) / 4) * 4;

	TArray<TPair<FVector, FVector>, TInlineAllocator<8 * (8 + 3)>> LineSegments;
	LineSegments.Reserve(SegmentNum * (SegmentNum + 3));

	for (int32 X = 0; X < SegmentNum; X++)
	{
		const float XA0 = ((float)(X + 0) / SegmentNum) * UE_TWO_PI;
		const float XA1 = ((float)(X + 1) / SegmentNum) * UE_TWO_PI;

		for (int32 Y = 0; Y < SegmentNum / 4; Y++)
		{
			const float YA0 = ((float)(Y + 0) / SegmentNum) * UE_TWO_PI;
			const float YA1 = ((float)(Y + 1) / SegmentNum) * UE_TWO_PI;

			const FVector V0T = Rotation.RotateVector(FVector(0.0f, 0.0f, +HalfLength) + FVector(FMath::Cos(XA0) * FMath::Sin(YA0), FMath::Sin(XA0) * FMath::Sin(YA0), +FMath::Cos(YA0)) * Radius) + Location;
			const FVector V1T = Rotation.RotateVector(FVector(0.0f, 0.0f, +HalfLength) + FVector(FMath::Cos(XA1) * FMath::Sin(YA0), FMath::Sin(XA1) * FMath::Sin(YA0), +FMath::Cos(YA0)) * Radius) + Location;
			const FVector V2T = Rotation.RotateVector(FVector(0.0f, 0.0f, +HalfLength) + FVector(FMath::Cos(XA0) * FMath::Sin(YA1), FMath::Sin(XA0) * FMath::Sin(YA1), +FMath::Cos(YA1)) * Radius) + Location;

			const FVector V0B = Rotation.RotateVector(FVector(0.0f, 0.0f, -HalfLength) + FVector(FMath::Cos(XA0) * FMath::Sin(YA0), FMath::Sin(XA0) * FMath::Sin(YA0), -FMath::Cos(YA0)) * Radius) + Location;
			const FVector V1B = Rotation.RotateVector(FVector(0.0f, 0.0f, -HalfLength) + FVector(FMath::Cos(XA1) * FMath::Sin(YA0), FMath::Sin(XA1) * FMath::Sin(YA0), -FMath::Cos(YA0)) * Radius) + Location;
			const FVector V2B = Rotation.RotateVector(FVector(0.0f, 0.0f, -HalfLength) + FVector(FMath::Cos(XA0) * FMath::Sin(YA1), FMath::Sin(XA0) * FMath::Sin(YA1), -FMath::Cos(YA1)) * Radius) + Location;

			LineSegments.Add({ V0T, V1T });
			LineSegments.Add({ V0T, V2T });
			LineSegments.Add({ V0B, V1B });
			LineSegments.Add({ V0B, V2B });
		}

		const float YA0 = ((float)(SegmentNum / 4) / SegmentNum) * UE_TWO_PI;

		const FVector V0T = Rotation.RotateVector(FVector(0.0f, 0.0f, +HalfLength) + FVector(FMath::Cos(XA0) * FMath::Sin(YA0), FMath::Sin(XA0) * FMath::Sin(YA0), +FMath::Cos(YA0)) * Radius) + Location;
		const FVector V1T = Rotation.RotateVector(FVector(0.0f, 0.0f, +HalfLength) + FVector(FMath::Cos(XA1) * FMath::Sin(YA0), FMath::Sin(XA1) * FMath::Sin(YA0), +FMath::Cos(YA0)) * Radius) + Location;

		const FVector V0B = Rotation.RotateVector(FVector(0.0f, 0.0f, -HalfLength) + FVector(FMath::Cos(XA0) * FMath::Sin(YA0), FMath::Sin(XA0) * FMath::Sin(YA0), -FMath::Cos(YA0)) * Radius) + Location;
		const FVector V1B = Rotation.RotateVector(FVector(0.0f, 0.0f, -HalfLength) + FVector(FMath::Cos(XA1) * FMath::Sin(YA0), FMath::Sin(XA1) * FMath::Sin(YA0), -FMath::Cos(YA0)) * Radius) + Location;

		LineSegments.Add({ V0T, V1T });
		LineSegments.Add({ V0B, V1B });

		const FVector E0 = Rotation.RotateVector(FVector(0.0f, 0.0f, +HalfLength) + FVector(FMath::Cos(XA0), FMath::Sin(XA0), 0.0f) * Radius) + Location;
		const FVector E1 = Rotation.RotateVector(FVector(0.0f, 0.0f, -HalfLength) + FVector(FMath::Cos(XA0), FMath::Sin(XA0), 0.0f) * Radius) + Location;
		LineSegments.Add({ E0, E1 });
	}

	DrawDebugLinesPairsArrayView(Drawer, LineSegments, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugCapsuleLine(const FDebugDrawer& Drawer, const FVector& StartLocation, const FVector& EndLocation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius, const int32 Segments)
{
	const FQuat Rotation = UE::DrawDebugLibrary::Private::FindOrientation(StartLocation, EndLocation);
	const float Length = (EndLocation - StartLocation).Length();

	DrawDebugCapsule(Drawer, StartLocation + Rotation.RotateVector(FVector(Length / 2.0f, 0.0f, 0.0f)), (Rotation * FQuat::MakeFromRotationVector(FVector(0.0f, UE_HALF_PI, 0.0f))).Rotator(), LineStyle, bDepthTest, Radius, Length / 2.0f, Segments);
}

void UDrawDebugLibrary::DrawDebugFrustum(const FDebugDrawer& Drawer, const FMatrix& FrustumToWorld, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest)
{
	FVector Vertices[2][2][2];
	for (int32 Z = 0; Z < 2; Z++)
	{
		for (int32 Y = 0; Y < 2; Y++)
		{
			for (int32 X = 0; X < 2; X++)
			{
				const FVector4 UnprojectedVertex = FrustumToWorld.TransformFVector4(
					FVector4(
						(X ? -1.0f : 1.0f),
						(Y ? -1.0f : 1.0f),
						(Z ? 0.0f : 1.0f),
						1.0f));
				Vertices[X][Y][Z] = FVector(UnprojectedVertex) / UnprojectedVertex.W;
			}
		}
	}

	DrawDebugLinesPairsArrayView(Drawer, {
		{ Vertices[0][0][0], Vertices[0][0][1] },
		{ Vertices[1][0][0], Vertices[1][0][1] },
		{ Vertices[0][1][0], Vertices[0][1][1] },
		{ Vertices[1][1][0], Vertices[1][1][1] },

		{ Vertices[0][0][0], Vertices[0][1][0] },
		{ Vertices[1][0][0], Vertices[1][1][0] },
		{ Vertices[0][0][1], Vertices[0][1][1] },
		{ Vertices[1][0][1], Vertices[1][1][1] },

		{ Vertices[0][0][0], Vertices[1][0][0] },
		{ Vertices[0][1][0], Vertices[1][1][0] },
		{ Vertices[0][0][1], Vertices[1][0][1] },
		{ Vertices[0][1][1], Vertices[1][1][1] },
		}, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugArrowHead(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const EDrawDebugArrowHead Type, const float Size)
{
	switch (Type)
	{
	case EDrawDebugArrowHead::None: { break; }

	case EDrawDebugArrowHead::Simple:
	{
		DrawDebugLinesPairsArrayView(Drawer, {
			{ Location, Location + Rotation.RotateVector(FVector(+Size, +Size, 0.0f)) },
			{ Location, Location + Rotation.RotateVector(FVector(+Size, -Size, 0.0f)) },
			}, LineStyle, bDepthTest);
		break;
	}

	case EDrawDebugArrowHead::FourWay:
	{
		DrawDebugLinesPairsArrayView(Drawer, {
			{ Location, Location + Rotation.RotateVector(FVector(+Size, +Size, 0)) },
			{ Location, Location + Rotation.RotateVector(FVector(+Size, -Size, 0)) },
			{ Location, Location + Rotation.RotateVector(FVector(+Size, 0, +Size)) },
			{ Location, Location + Rotation.RotateVector(FVector(+Size, 0, -Size)) },
			}, LineStyle, bDepthTest);
		break;
	}

	case EDrawDebugArrowHead::TriangularPyramid:
	{
		DrawDebugTriangularBasePyramid(Drawer,
			Location + Rotation.RotateVector(FVector(+Size / 2, 0, 0)),
			(Rotation.Quaternion() * FQuat::MakeFromRotationVector(FVector(0, -UE_PI / 2, 0))).Rotator(),
			LineStyle, bDepthTest, Size, Size);
		break;
	}

	case EDrawDebugArrowHead::SquarePyramid:
	{
		DrawDebugSquareBasePyramid(Drawer,
			Location + Rotation.RotateVector(FVector(+Size / 2, 0, 0)),
			(Rotation.Quaternion() * FQuat::MakeFromRotationVector(FVector(0, -UE_PI / 2, 0))).Rotator(),
			LineStyle, bDepthTest, Size, Size);
		break;
	}

	case EDrawDebugArrowHead::Cone:
	{
		DrawDebugCone(Drawer,
			Location + Rotation.RotateVector(FVector(+Size / 2, 0, 0)),
			(Rotation.Quaternion() * FQuat::MakeFromRotationVector(FVector(0, -UE_PI / 2, 0))).Rotator(),
			LineStyle, bDepthTest, Size, Size / 2, 7);
		break;
	}

	case EDrawDebugArrowHead::Triangle:
	{
		DrawDebugLinesPairsArrayView(Drawer, {
			{ Location, Location + Rotation.RotateVector(FVector(+Size, +Size / 2.0f, 0.0f)) },
			{ Location, Location + Rotation.RotateVector(FVector(+Size, -Size / 2.0f, 0.0f)) },
			{
				Location + Rotation.RotateVector(FVector(+Size, -Size / 2.0f, 0.0f)),
				Location + Rotation.RotateVector(FVector(+Size, +Size / 2.0f, 0.0f)) },
			}, LineStyle, bDepthTest);
		break;
	}

	case EDrawDebugArrowHead::Square:
	{
		DrawDebugSquare(Drawer, Location, Rotation, LineStyle, bDepthTest, Size);
		break;
	}

	case EDrawDebugArrowHead::Diamond:
	{
		DrawDebugDiamond(Drawer, Location, Rotation, LineStyle, bDepthTest, Size);
		break;
	}

	case EDrawDebugArrowHead::Circle:
	{
		DrawDebugCircle(Drawer, Location, Rotation, LineStyle, bDepthTest, Size);
		break;
	}

	case EDrawDebugArrowHead::Sphere:
	{
		DrawDebugSimpleSphere(Drawer, Location, Rotation, LineStyle, bDepthTest, Size);
		break;
	}

	case EDrawDebugArrowHead::Box:
	{
		DrawDebugBox(Drawer, Location, Rotation, LineStyle, bDepthTest, FVector(Size, Size, Size));
		break;
	}

	default: checkNoEntry();
	}
}

void UDrawDebugLibrary::DrawDebugArrow(
	const FDebugDrawer& Drawer,
	const FVector& StartLocation,
	const FVector& EndLocation,
	const FDrawDebugLineStyle& LineStyle,
	const bool bDepthTest,
	const FDrawDebugArrowSettings& Settings)
{
	const FQuat Rotation = UE::DrawDebugLibrary::Private::FindOrientation(StartLocation, EndLocation);

	return DrawDebugOrientedArrow(Drawer, StartLocation, Rotation.Rotator(), (EndLocation - StartLocation).Length(), LineStyle, bDepthTest, Settings);
}

void UDrawDebugLibrary::DrawDebugOrientedArrow(
	const FDebugDrawer& Drawer, 
	const FVector& Location, 
	const FRotator& Rotation, 
	const float Length,
	const FDrawDebugLineStyle& LineStyle,
	const bool bDepthTest,
	const FDrawDebugArrowSettings& Settings)
{
	const FVector HeadStartLocation = Location;
	const FVector HeadEndLocation = Location + Rotation.RotateVector(FVector(Length, 0.0f, 0.0f));

	const FVector LineStartLocation = Settings.bArrowLineEndsAtStartHead ? 
		Location + Rotation.RotateVector(FVector(Settings.ArrowHeadStartSize, 0.0f, 0.0f)) : HeadStartLocation;
	
	const FVector LineEndLocation = Settings.bArrowLineEndsAtEndHead ?
		Location + Rotation.RotateVector(FVector(Length - Settings.ArrowHeadEndSize, 0.0f, 0.0f)) : HeadEndLocation;

	DrawDebugLine(Drawer, LineStartLocation, LineEndLocation, LineStyle, bDepthTest);

	const FRotator HeadStartRotation = Rotation;
	const FRotator HeadEndRotation = (Rotation.Quaternion() * FQuat::MakeFromRotationVector(FVector(0.0f, 0.0f, UE_PI))).Rotator();

	if (Settings.bArrowheadOnStart)
	{
		DrawDebugArrowHead(Drawer, HeadStartLocation, HeadStartRotation, LineStyle, bDepthTest, Settings.ArrowHeadStartType, Settings.ArrowHeadStartSize);
	}

	if (Settings.bArrowheadOnEnd)
	{
		DrawDebugArrowHead(Drawer, HeadEndLocation, HeadEndRotation, LineStyle, bDepthTest, Settings.ArrowHeadEndType, Settings.ArrowHeadEndSize);
	}
}

void UDrawDebugLibrary::DrawDebugGroundTargetArrow(const FDebugDrawer& Drawer, const FVector& Location, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Length, const float ArrowHeadSize, const float Radius, const int32 Segements)
{
	FDrawDebugArrowSettings Settings;
	Settings.ArrowHeadEndType = EDrawDebugArrowHead::FourWay;

	DrawDebugArrow(
		Drawer,
		Location + FVector(0.0f, 0.0f, Length),
		Location,
		LineStyle,
		bDepthTest,
		Settings);

	DrawDebugCircle(Drawer, Location, FRotator::ZeroRotator, LineStyle, bDepthTest, Radius, Segements);
}

void UDrawDebugLibrary::DrawDebugFlatArrow(
	const FDebugDrawer& Drawer,
	const FVector& Location,
	const FRotator& Rotation,
	const FDrawDebugLineStyle& LineStyle,
	const bool bDepthTest,
	const float Length,
	const float Width)
{
	const FVector V0 = Location - Rotation.RotateVector(FVector(0, Width / 4.0f, 0));
	const FVector V1 = Location + Rotation.RotateVector(FVector(0, Width / 4.0f, 0));
	const FVector V2 = V0 + Rotation.RotateVector(FVector(Length / 3.0f, 0, 0));
	const FVector V3 = V1 + Rotation.RotateVector(FVector(Length / 3.0f, 0, 0));
	const FVector V4 = V2 - Rotation.RotateVector(FVector(0, Width / 4.0f, 0));
	const FVector V5 = V3 + Rotation.RotateVector(FVector(0, Width / 4.0f, 0));
	const FVector V6 = Location + Rotation.RotateVector(FVector(Length, 0, 0));

	DrawDebugLinesPairsArrayView(Drawer, {
		{ V0, V1 },
		{ V0, V2 },
		{ V1, V3 },
		{ V2, V4 },
		{ V3, V5 },
		{ V4, V6 },
		{ V5, V6 },
		}, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugCircleArrow(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const float Angle, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius, const float Length, const FDrawDebugArrowSettings& Settings)
{
	const FVector Start = Location + Rotation.RotateVector(Radius * FVector(FMath::Cos(FMath::DegreesToRadians(Angle)), FMath::Sin(FMath::DegreesToRadians(Angle)), 0.0f));
	const FVector End = Location + Rotation.RotateVector((Radius + Length) * FVector(FMath::Cos(FMath::DegreesToRadians(Angle)), FMath::Sin(FMath::DegreesToRadians(Angle)), 0.0f));
	DrawDebugArrow(Drawer, Start, End, LineStyle, bDepthTest, Settings);
}

void UDrawDebugLibrary::DrawDebugArcArrow(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const float Angle, const FDrawDebugLineStyle& LineStyle , const bool bDepthTest, const float Radius, const int32 Segments, const FDrawDebugArrowSettings& Settings)
{
	const float AngleStartOffset = Settings.bArrowLineEndsAtStartHead ? FMath::Min(Settings.ArrowHeadStartSize / FMath::Max(Radius, UE_SMALL_NUMBER), FMath::DegreesToRadians(Angle) / 2.0f) : 0.0f;
	const float AngleEndOffset = Settings.bArrowLineEndsAtEndHead ? FMath::Min(Settings.ArrowHeadEndSize / FMath::Max(Radius, UE_SMALL_NUMBER), FMath::DegreesToRadians(Angle) / 2.0f) : 0.0f;
	const FRotator ArcRotation = (Rotation.Quaternion() * FQuat::MakeFromRotationVector(FVector(0.0f, 0.0f, AngleStartOffset))).Rotator();

	DrawDebugArc(Drawer, Location, ArcRotation, Angle - FMath::RadiansToDegrees(AngleStartOffset) - FMath::RadiansToDegrees(AngleEndOffset), LineStyle, bDepthTest, Radius, Segments);

	if (Settings.bArrowheadOnStart)
	{
		DrawDebugArrowHead(
			Drawer,
			Location + Rotation.RotateVector(Radius * FVector::ForwardVector),
			(Rotation.Quaternion() * FQuat::MakeFromRotationVector(FVector(0.0f, 0.0f, UE_PI / 2.0f))).Rotator(),
			LineStyle,
			bDepthTest,
			Settings.ArrowHeadStartType,
			Settings.ArrowHeadStartSize);
	}

	if (Settings.bArrowheadOnEnd)
	{
		DrawDebugArrowHead(
			Drawer,
			Location + Rotation.RotateVector(Radius * FVector(FMath::Cos(FMath::DegreesToRadians(Angle)), FMath::Sin(FMath::DegreesToRadians(Angle)), 0.0f)),
			(Rotation.Quaternion() * FQuat::MakeFromRotationVector(FVector(0.0f, 0.0f, FMath::DegreesToRadians(Angle) - UE_PI / 2.0f))).Rotator(),
			LineStyle,
			bDepthTest,
			Settings.ArrowHeadEndType,
			Settings.ArrowHeadEndSize);
	}
}

void UDrawDebugLibrary::DrawDebugCatmullRomSplineStart(const FDebugDrawer& Drawer, const FVector& V0, const FVector& V1, const FVector& V2, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const bool bMonotonic, const int32 Segments)
{
	const int32 SegmentNum = FMath::Max(Segments, 2);

	TArray<TPair<FVector, FVector>, TInlineAllocator<14>> LineSegments;
	LineSegments.Reserve(SegmentNum - 1);

	if (bMonotonic)
	{
		for (int32 Idx = 0; Idx < Segments - 1; Idx++)
		{
			const float Alpha0 = ((float)(Idx + 0)) / (Segments - 1);
			const float Alpha1 = ((float)(Idx + 1)) / (Segments - 1);
			const FVector Location0 = UE::DrawDebugLibrary::Private::InterpolateCubicMonoStart(V0, V1, V2, Alpha0);
			const FVector Location1 = UE::DrawDebugLibrary::Private::InterpolateCubicMonoStart(V0, V1, V2, Alpha1);
			LineSegments.Add({ Location0, Location1 });
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < Segments - 1; Idx++)
		{
			const float Alpha0 = ((float)(Idx + 0)) / (Segments - 1);
			const float Alpha1 = ((float)(Idx + 1)) / (Segments - 1);
			const FVector Location0 = UE::DrawDebugLibrary::Private::InterpolateCubicStart(V0, V1, V2, Alpha0);
			const FVector Location1 = UE::DrawDebugLibrary::Private::InterpolateCubicStart(V0, V1, V2, Alpha1);
			LineSegments.Add({ Location0, Location1 });
		}
	}

	DrawDebugLinesPairsArrayView(Drawer, LineSegments, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugCatmullRomSplineEnd(const FDebugDrawer& Drawer, const FVector& V0, const FVector& V1, const FVector& V2, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const bool bMonotonic, const int32 Segments)
{
	const int32 SegmentNum = FMath::Max(Segments, 2);

	TArray<TPair<FVector, FVector>, TInlineAllocator<14>> LineSegments;
	LineSegments.Reserve(SegmentNum - 1);

	if (bMonotonic)
	{
		for (int32 Idx = 0; Idx < Segments - 1; Idx++)
		{
			const float Alpha0 = ((float)(Idx + 0)) / (Segments - 1);
			const float Alpha1 = ((float)(Idx + 1)) / (Segments - 1);
			const FVector Location0 = UE::DrawDebugLibrary::Private::InterpolateCubicMonoEnd(V0, V1, V2, Alpha0);
			const FVector Location1 = UE::DrawDebugLibrary::Private::InterpolateCubicMonoEnd(V0, V1, V2, Alpha1);
			LineSegments.Add({ Location0, Location1 });
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < Segments - 1; Idx++)
		{
			const float Alpha0 = ((float)(Idx + 0)) / (Segments - 1);
			const float Alpha1 = ((float)(Idx + 1)) / (Segments - 1);
			const FVector Location0 = UE::DrawDebugLibrary::Private::InterpolateCubicEnd(V0, V1, V2, Alpha0);
			const FVector Location1 = UE::DrawDebugLibrary::Private::InterpolateCubicEnd(V0, V1, V2, Alpha1);
			LineSegments.Add({ Location0, Location1 });
		}
	}

	DrawDebugLinesPairsArrayView(Drawer, LineSegments, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugCatmullRomSplineSection(const FDebugDrawer& Drawer, const FVector& V0, const FVector& V1, const FVector& V2, const FVector& V3, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const bool bMonotonic, const int32 Segments)
{
	const int32 SegmentNum = FMath::Max(Segments, 2);

	TArray<TPair<FVector, FVector>, TInlineAllocator<14>> LineSegments;
	LineSegments.Reserve(SegmentNum - 1);

	if (bMonotonic)
	{
		for (int32 Idx = 0; Idx < Segments - 1; Idx++)
		{
			const float Alpha0 = ((float)(Idx + 0)) / (Segments - 1);
			const float Alpha1 = ((float)(Idx + 1)) / (Segments - 1);
			const FVector Location0 = UE::DrawDebugLibrary::Private::InterpolateCubicMono(V0, V1, V2, V3, Alpha0);
			const FVector Location1 = UE::DrawDebugLibrary::Private::InterpolateCubicMono(V0, V1, V2, V3, Alpha1);
			LineSegments.Add({ Location0, Location1 });
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < Segments - 1; Idx++)
		{
			const float Alpha0 = ((float)(Idx + 0)) / (Segments - 1);
			const float Alpha1 = ((float)(Idx + 1)) / (Segments - 1);
			const FVector Location0 = UE::DrawDebugLibrary::Private::InterpolateCubic(V0, V1, V2, V3, Alpha0);
			const FVector Location1 = UE::DrawDebugLibrary::Private::InterpolateCubic(V0, V1, V2, V3, Alpha1);
			LineSegments.Add({ Location0, Location1 });
		}
	}

	DrawDebugLinesPairsArrayView(Drawer, LineSegments, LineStyle, bDepthTest);
}


void UDrawDebugLibrary::DrawDebugCatmullRomSpline(const FDebugDrawer& Drawer, const TArray<FVector>& Points, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const bool bMonotonic, const int32 Segments)
{
	DrawDebugCatmullRomSplineArrayView(Drawer, Points, LineStyle, bDepthTest, bMonotonic, Segments);
}

void UDrawDebugLibrary::DrawDebugCatmullRomSplineArrayView(const FDebugDrawer& Drawer, const TArrayView<const FVector> Points, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const bool bMonotonic, const int32 Segments)
{
	const int32 Num = Points.Num();
	
	if (Num == 0) { return; }
	if (Num == 1)
	{
		DrawDebugPoint(Drawer, Points[0], DrawDebugPointStyleFromLineStyle(LineStyle), bDepthTest);
		return;
	}

	if (Num == 2)
	{
		DrawDebugLine(Drawer, Points[0], Points[1], LineStyle, bDepthTest);
		return;
	}

	DrawDebugCatmullRomSplineStart(Drawer, Points[0], Points[1], Points[2], LineStyle, bDepthTest, bMonotonic, Segments);

	for (int32 Idx = 0; Idx < Num - 3; Idx++)
	{
		DrawDebugCatmullRomSplineSection(Drawer, Points[Idx + 0], Points[Idx + 1], Points[Idx + 2], Points[Idx + 3], LineStyle, bDepthTest, bMonotonic, Segments);
	}

	DrawDebugCatmullRomSplineEnd(Drawer, Points[Num - 3], Points[Num - 2], Points[Num - 1], LineStyle, bDepthTest, bMonotonic, Segments);
}



void UDrawDebugLibrary::DrawDebugAngle(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const float Angle, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float LineLength, const float AngleRadius, const int32 Segments)
{
	DrawDebugArc(Drawer, Location, Rotation, Angle, LineStyle, bDepthTest, AngleRadius, Segments);
	DrawDebugLinesPairsArrayView(Drawer, {
		{ Location, Location + Rotation.RotateVector(FVector(LineLength, 0, 0)) },
		{ Location, Location + Rotation.RotateVector(LineLength * FVector(FMath::Cos(FMath::DegreesToRadians(Angle)), FMath::Sin(FMath::DegreesToRadians(Angle)), 0)) }
		}, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugAngleBetween(const FDebugDrawer& Drawer, const FVector& P0, const FVector& P1, const FVector& P2, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float AngleRadius, const int32 Segments)
{
	const float Angle = FMath::Acos((P0 - P1).GetSafeNormal().Dot((P2 - P1).GetSafeNormal()));

	FVector AxisX = (P0 - P1).GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector);
	FVector AxisY = (P2 - P1).GetSafeNormal(UE_SMALL_NUMBER, FVector::RightVector);
	FVector AxisZ = AxisX.Cross(AxisY).GetSafeNormal(UE_SMALL_NUMBER, FVector::UpVector);
	AxisY = AxisZ.Cross(AxisX).GetSafeNormal(UE_SMALL_NUMBER, FVector::RightVector);
	
	FMatrix RotationMatrix = FMatrix::Identity;
	RotationMatrix.SetAxis(0, AxisX);
	RotationMatrix.SetAxis(1, AxisY);
	RotationMatrix.SetAxis(2, AxisZ);

	DrawDebugArc(Drawer, P1, RotationMatrix.ToQuat().Rotator(), FMath::RadiansToDegrees(Angle), LineStyle, bDepthTest, AngleRadius, Segments);
	DrawDebugLinesPairsArrayView(Drawer, {{ P0, P1 }, { P1, P2 }}, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugLocation(const FDebugDrawer& Drawer, const FVector& Location, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float DrawRadius, const int32 Segments)
{
	DrawDebugSphere(Drawer, Location, FRotator::ZeroRotator, LineStyle, bDepthTest, DrawRadius, Segments);
}

void UDrawDebugLibrary::DrawDebugLocations(const FDebugDrawer& Drawer, const TArray<FVector>& Locations, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float DrawRadius, const int32 Segments)
{
	DrawDebugLocationsArrayView(Drawer, Locations, LineStyle, bDepthTest, DrawRadius, Segments);
}

void UDrawDebugLibrary::DrawDebugLocationsArrayView(const FDebugDrawer& Drawer, const TArrayView<const FVector> Locations, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float DrawRadius, const int32 Segments)
{
	for (const FVector& Location : Locations)
	{
		DrawDebugLocation(Drawer, Location, LineStyle, bDepthTest, DrawRadius, Segments);
	}
}

void UDrawDebugLibrary::DrawDebugRotation(
	const FDebugDrawer& Drawer,
	const FVector& Location,
	const FRotator& Rotation,
	const FDrawDebugLineStyle& LineStyle,
	const bool bDepthTest,
	const float DrawRadius)
{
	DrawDebugLine(Drawer, Location, Location + Rotation.RotateVector(FVector(DrawRadius, 0.0f, 0.0f)), DrawDebugLineStyleWithColorNoOpacity(LineStyle, FLinearColor::Red), bDepthTest);
	DrawDebugLine(Drawer, Location, Location + Rotation.RotateVector(FVector(0.0f, DrawRadius, 0.0f)), DrawDebugLineStyleWithColorNoOpacity(LineStyle, FLinearColor::Green), bDepthTest);
	DrawDebugLine(Drawer, Location, Location + Rotation.RotateVector(FVector(0.0f, 0.0f, DrawRadius)), DrawDebugLineStyleWithColorNoOpacity(LineStyle, FLinearColor::Blue), bDepthTest);
}

void UDrawDebugLibrary::DrawDebugDirection(const FDebugDrawer& Drawer, const FVector& Location, const FVector& Direction, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float DrawArrowLength, const float ArrowHeadScale)
{
	DrawDebugArrow(Drawer, Location, Location + DrawArrowLength * Direction, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugVelocity(const FDebugDrawer& Drawer, const FVector& Location, const FVector& Velocity, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float DrawVelocityLineScale)
{
	DrawDebugLine(Drawer, Location, Location + DrawVelocityLineScale * Velocity, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugTransform(const FDebugDrawer& Drawer, const FTransform& Transform, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float DrawRadius)
{
	const FVector Location = Transform.GetLocation();
	const FQuat Rotation = Transform.GetRotation();
	const FVector Scale = Transform.GetScale3D();

	DrawDebugLine(Drawer, Location, Location + Rotation.RotateVector(Scale * FVector(DrawRadius, 0, 0)), DrawDebugLineStyleWithColorNoOpacity(LineStyle, FLinearColor::Red), bDepthTest);
	DrawDebugLine(Drawer, Location, Location + Rotation.RotateVector(Scale * FVector(0, DrawRadius, 0)), DrawDebugLineStyleWithColorNoOpacity(LineStyle, FLinearColor::Green), bDepthTest);
	DrawDebugLine(Drawer, Location, Location + Rotation.RotateVector(Scale * FVector(0, 0, DrawRadius)), DrawDebugLineStyleWithColorNoOpacity(LineStyle, FLinearColor::Blue), bDepthTest);
}

void UDrawDebugLibrary::DrawDebugEvent(const FDebugDrawer& Drawer, bool bTimeUntilEventKnown, const float TimeUntilEvent, const FVector& Location, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Size)
{
	DrawDebugLine(
		Drawer,
		Location + FVector(0, 0, +Size),
		Location + FVector(0, 0, -Size),
		LineStyle, bDepthTest);

	DrawDebugSphere(Drawer, Location, FRotator::ZeroRotator, LineStyle, bDepthTest, Size * 0.1f, 8);

	if (bTimeUntilEventKnown && TimeUntilEvent < 1.0f && TimeUntilEvent > -1.0f)
	{
		DrawDebugSphere(Drawer, Location + FVector(0, 0, TimeUntilEvent * Size), FRotator::ZeroRotator, LineStyle, bDepthTest, Size * 0.25f, 8);
	}
}


void UDrawDebugLibrary::DrawDebugChair(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const FDrawDebugChairSettings& Settings)
{
	const float HalfWidth = Settings.Width / 2.0f;
	const FVector F0 = Location + Rotation.RotateVector(FVector(+HalfWidth, +HalfWidth, 0));
	const FVector F1 = Location + Rotation.RotateVector(FVector(+HalfWidth, -HalfWidth, 0));
	const FVector F2 = Location + Rotation.RotateVector(FVector(-HalfWidth, -HalfWidth, 0));
	const FVector F3 = Location + Rotation.RotateVector(FVector(-HalfWidth, +HalfWidth, 0));

	const FVector S0 = Location + Rotation.RotateVector(FVector(+HalfWidth, +HalfWidth, Settings.SeatHeight));
	const FVector S1 = Location + Rotation.RotateVector(FVector(+HalfWidth, -HalfWidth, Settings.SeatHeight));
	const FVector S2 = Location + Rotation.RotateVector(FVector(-HalfWidth, -HalfWidth, Settings.SeatHeight));
	const FVector S3 = Location + Rotation.RotateVector(FVector(-HalfWidth, +HalfWidth, Settings.SeatHeight));

	const FVector B2 = Location + Rotation.RotateVector(FVector(-HalfWidth - Settings.BackTilt, -HalfWidth, Settings.SeatHeight + Settings.BackHeight));
	const FVector B3 = Location + Rotation.RotateVector(FVector(-HalfWidth - Settings.BackTilt, +HalfWidth, Settings.SeatHeight + Settings.BackHeight));

	DrawDebugLinesPairsArrayView(Drawer, {
		{ F0, S0 },
		{ F1, S1 },
		{ F2, S2 },
		{ F3, S3 },

		{ S0, S1 },
		{ S1, S2 },
		{ S2, S3 },
		{ S3, S0 },

		{ S2, B2 },
		{ S3, B3 },
		{ B2, B3 },
		}, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugDoor(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const FDrawDebugDoorSettings& Settings)
{
	const float HalfWidth = Settings.Width / 2.0f;

	const FVector F0 = Location + Rotation.RotateVector(FVector(0.0f, -HalfWidth, 0.0f));
	const FVector F1 = Location + Rotation.RotateVector(FVector(0.0f, +HalfWidth, 0.0f));
	const FVector F2 = Location + Rotation.RotateVector(FVector(0.0f, +HalfWidth, Settings.Height));
	const FVector F3 = Location + Rotation.RotateVector(FVector(0.0f, -HalfWidth, Settings.Height));

	const FVector LI0 = FVector(0.0f, -HalfWidth + Settings.Inset, Settings.Inset);
	const FVector LI1 = FVector(0.0f, +HalfWidth - Settings.Inset, Settings.Inset);
	const FVector LI2 = FVector(0.0f, +HalfWidth - Settings.Inset, Settings.Height - Settings.Inset);
	const FVector LI3 = FVector(0.0f, -HalfWidth + Settings.Inset, Settings.Height - Settings.Inset);

	const FVector OpenOffset = Settings.bHandleOnLeft ?
		FVector(0.0f, +HalfWidth - Settings.Inset, 0.0f) :
		FVector(0.0f, -HalfWidth + Settings.Inset, 0.0f);

	const FQuat OpenRotation = FQuat::MakeFromRotationVector(FVector(0.0f, 0.0f, FMath::DegreesToRadians(-Settings.OpenAngle)));

	const FVector I0 = Location + Rotation.RotateVector(OpenRotation.RotateVector(LI0 - OpenOffset) + OpenOffset);
	const FVector I1 = Location + Rotation.RotateVector(OpenRotation.RotateVector(LI1 - OpenOffset) + OpenOffset);
	const FVector I2 = Location + Rotation.RotateVector(OpenRotation.RotateVector(LI2 - OpenOffset) + OpenOffset);
	const FVector I3 = Location + Rotation.RotateVector(OpenRotation.RotateVector(LI3 - OpenOffset) + OpenOffset);

	DrawDebugLinesPairsArrayView(Drawer, {
		{ F0, F1 },
		{ F1, F2 },
		{ F2, F3 },
		{ F3, F0 },

		{ I0, I1 },
		{ I1, I2 },
		{ I2, I3 },
		{ I3, I0 },
		}, LineStyle, bDepthTest);

	if (Settings.bOuterHandle)
	{
		const FVector LOuterHandle = Settings.bHandleOnLeft ?
			FVector(+Settings.HandleRadius + Settings.HandleExtension, Settings.HandleOffset - Settings.Width, Settings.HandleHeight) :
			FVector(+Settings.HandleRadius + Settings.HandleExtension, Settings.Width - Settings.HandleOffset, Settings.HandleHeight);
		const FVector OuterHandle = Location + Rotation.RotateVector(OpenRotation.RotateVector(LOuterHandle) + OpenOffset);

		DrawDebugSimpleSphere(Drawer, OuterHandle, FRotator::ZeroRotator, DrawDebugLineStyleWithType(LineStyle, EDrawDebugLineType::Solid), bDepthTest, Settings.HandleRadius);
	}

	if (Settings.bInnerHandle)
	{
		const FVector LInnerHandle = Settings.bHandleOnLeft ?
			FVector(-Settings.HandleRadius - Settings.HandleExtension, Settings.HandleOffset - Settings.Width, Settings.HandleHeight) :
			FVector(-Settings.HandleRadius - Settings.HandleExtension, Settings.Width - Settings.HandleOffset, Settings.HandleHeight);
		const FVector InnerHandle = Location + Rotation.RotateVector(OpenRotation.RotateVector(LInnerHandle) + OpenOffset);

		DrawDebugSimpleSphere(Drawer, InnerHandle, FRotator::ZeroRotator, DrawDebugLineStyleWithType(LineStyle, EDrawDebugLineType::Solid), bDepthTest, Settings.HandleRadius);
	}

	if (Settings.bDrawEntryArrow)
	{
		DrawDebugArrow(
			Drawer,
			Location + Rotation.RotateVector(FVector(-50.0f, 0.0f, 2.0f)),
			Location + Rotation.RotateVector(FVector(-10.0f, 0.0f, 2.0f)),
			LineStyle,
			bDepthTest);
	}
}

void UDrawDebugLibrary::DrawDebugCamera(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, float Scale, float FOVDegrees)
{
	DrawDebugRotation(Drawer, Location, Rotation, LineStyle, bDepthTest, Scale);
	DrawDebugBox(Drawer, Location, Rotation, LineStyle, bDepthTest, FVector(Scale, Scale, Scale));

	// draw "lens" portion
	const FRotationTranslationMatrix Axes(Rotation, Location);
	const FVector XAxis = Axes.GetScaledAxis(EAxis::X);
	const FVector YAxis = Axes.GetScaledAxis(EAxis::Y);
	const FVector ZAxis = Axes.GetScaledAxis(EAxis::Z);

	const FVector LensPoint = Location + XAxis * Scale;
	const float HalfLensSize = Scale * FMath::Tan(FMath::DegreesToRadians(FOVDegrees * 0.5f));

	const FVector C0 = LensPoint + XAxis * Scale+ (YAxis * HalfLensSize) + (ZAxis * HalfLensSize);
	const FVector C1 = LensPoint + XAxis * Scale+ (YAxis * HalfLensSize) - (ZAxis * HalfLensSize);
	const FVector C2 = LensPoint + XAxis * Scale- (YAxis * HalfLensSize) - (ZAxis * HalfLensSize);
	const FVector C3 = LensPoint + XAxis * Scale- (YAxis * HalfLensSize) + (ZAxis * HalfLensSize);

	DrawDebugLinesPairsArrayView(Drawer, {
		{ LensPoint, C0 },
		{ LensPoint, C1 },
		{ LensPoint, C2 },
		{ LensPoint, C3 },

		{ C0, C1 },
		{ C1, C2 },
		{ C2, C3 },
		{ C3, C0 },
		}, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugBackpack(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const FDrawDebugBackpackSettings& Settings)
{
	DrawDebugBox(
		Drawer, 
		Location + Rotation.RotateVector(FVector(Settings.Depth / 2, 0, 0)), 
		Rotation, 
		LineStyle, 
		bDepthTest, 
		FVector(Settings.Depth / 2, Settings.Width / 2, Settings.Height / 2));

	const float W = Settings.Width;
	const float H = Settings.Height;
	const float SD = Settings.StrapDistance;
	const float SW = Settings.StrapWidth;
	const float SHI = Settings.StrapHorizontalInset;
	const float SVI = Settings.StrapVerticalInset;
	const float SCH = Settings.StrapContactHeight;

	DrawDebugCatmullRomSplineArrayView(Drawer, {
		Location + Rotation.RotateVector(FVector(0.0f, -W / 2 + SHI, +H / 2 - SVI)),
		Location + Rotation.RotateVector(FVector( -SD, -W / 2 + SHI, -H / 2 + SCH)),
		Location + Rotation.RotateVector(FVector(0.0f, -W / 2 + SHI, -H / 2 + SVI)),
		}, LineStyle, bDepthTest);

	DrawDebugCatmullRomSplineArrayView(Drawer, {
		Location + Rotation.RotateVector(FVector(0.0f, -W / 2 + SHI + SW, +H / 2 - SVI)),
		Location + Rotation.RotateVector(FVector( -SD, -W / 2 + SHI + SW, -H / 2 + SCH)),
		Location + Rotation.RotateVector(FVector(0.0f, -W / 2 + SHI + SW, -H / 2 + SVI)),
		}, LineStyle, bDepthTest);

	DrawDebugCatmullRomSplineArrayView(Drawer, {
		Location + Rotation.RotateVector(FVector(0.0f, +W / 2 - SHI, +H / 2 - SVI)),
		Location + Rotation.RotateVector(FVector( -SD, +W / 2 - SHI, -H / 2 + SCH)),
		Location + Rotation.RotateVector(FVector(0.0f, +W / 2 - SHI, -H / 2 + SVI)),
		}, LineStyle, bDepthTest);

	DrawDebugCatmullRomSplineArrayView(Drawer, {
		Location + Rotation.RotateVector(FVector(0.0f, +W / 2 - SHI - SW, +H / 2 - SVI)),
		Location + Rotation.RotateVector(FVector( -SD, +W / 2 - SHI - SW, -H / 2 + SCH)),
		Location + Rotation.RotateVector(FVector(0.0f, +W / 2 - SHI - SW, -H / 2 + SVI)),
		}, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugBriefcase(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const FDrawDebugBriefcaseSettings& Settings)
{
	DrawDebugBox(
		Drawer,
		Location + Rotation.RotateVector(FVector(0, 0, -Settings.HalfExtents.Z - Settings.HandleHeight)),
		Rotation,
		LineStyle,
		bDepthTest,
		Settings.HalfExtents);

	DrawDebugCatmullRomSplineArrayView(Drawer, {
		Location + Rotation.RotateVector(FVector(+Settings.HandleLength / 2, -Settings.HandleWidth / 2, -Settings.HandleHeight)),
		Location + Rotation.RotateVector(FVector(+Settings.HandleLength / 2 - Settings.HandleLength / 8, -Settings.HandleWidth / 2, 0.0f)),
		Location + Rotation.RotateVector(FVector(-Settings.HandleLength / 2 + Settings.HandleLength / 8, -Settings.HandleWidth / 2, 0.0f)),
		Location + Rotation.RotateVector(FVector(-Settings.HandleLength / 2, -Settings.HandleWidth / 2, -Settings.HandleHeight)),
		}, LineStyle, bDepthTest, true);

	DrawDebugCatmullRomSplineArrayView(Drawer, {
		Location + Rotation.RotateVector(FVector(+Settings.HandleLength / 2, +Settings.HandleWidth / 2, -Settings.HandleHeight)),
		Location + Rotation.RotateVector(FVector(+Settings.HandleLength / 2 - Settings.HandleLength / 8, +Settings.HandleWidth / 2, 0.0f)),
		Location + Rotation.RotateVector(FVector(-Settings.HandleLength / 2 + Settings.HandleLength / 8, +Settings.HandleWidth / 2, 0.0f)),
		Location + Rotation.RotateVector(FVector(-Settings.HandleLength / 2, +Settings.HandleWidth / 2, -Settings.HandleHeight)),
		}, LineStyle, bDepthTest, true);
}

void UDrawDebugLibrary::DrawDebugTrajectory(const FDebugDrawer& Drawer, const TArray<FVector>& Locations, const TArray<FVector>& Directions, const FTransform& RelativeTransform, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float DrawArrowLength, const float PointRadius, const float ArrowHeadScale, const int32 Segments, const float VerticalOffset)
{
	DrawDebugTrajectoryFromArrayViews(Drawer, Locations, Directions, RelativeTransform, LineStyle, bDepthTest, DrawArrowLength, PointRadius, ArrowHeadScale, Segments, VerticalOffset);
}

void UDrawDebugLibrary::DrawDebugTrajectoryFromArrayViews(const FDebugDrawer& Drawer, const TArrayView<const FVector> Locations, const TArrayView<const FVector> Directions, const FTransform& RelativeTransform, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float DrawArrowLength, const float PointRadius, const float ArrowHeadScale, const int32 Segments, const float VerticalOffset)
{
	const int32 SampleNum = FMath::Min(Locations.Num(), Directions.Num());

	for (int32 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
	{
		const FVector Location = RelativeTransform.TransformPosition(Locations[SampleIdx]) + FVector(0.0f, 0.0f, VerticalOffset);
		const FVector Direction = RelativeTransform.TransformVectorNoScale(Directions[SampleIdx]);

		DrawDebugSimpleSphere(Drawer, Location, FRotator::ZeroRotator, LineStyle, bDepthTest, PointRadius, Segments);
		DrawDebugArrow(Drawer, Location, Location + DrawArrowLength * Direction, LineStyle, bDepthTest);

		if (SampleIdx > 0)
		{
			const FVector PrevLocation = RelativeTransform.TransformPosition(Locations[SampleIdx - 1]) + FVector(0.0f, 0.0f, VerticalOffset);
			DrawDebugLine(Drawer, Location, PrevLocation, LineStyle, bDepthTest);
		}
	}
}

void UDrawDebugLibrary::DrawDebugTransformTrajectory(const FDebugDrawer& Drawer, const FTransformTrajectory& TransformTrajectory, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float Radius, const float VerticalOffset)
{

	FTransformTrajectorySample TimeZeroSample = TransformTrajectory.GetSampleAtTime(0.0f);
	DrawDebugSphere(Drawer, TimeZeroSample.Position + FVector(0.0f, 0.0f, VerticalOffset), FRotator::ZeroRotator, LineStyle, bDepthTest, Radius);

	const int32 SampleNum = TransformTrajectory.Samples.Num();
	for (int32 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
	{
		const FVector Location = TransformTrajectory.Samples[SampleIdx].Position + FVector(0.0f, 0.0f, VerticalOffset);
		const FQuat Rotation = TransformTrajectory.Samples[SampleIdx].Facing;

		DrawDebugRotation(Drawer, Location, Rotation.Rotator(), LineStyle, bDepthTest, Radius);
		DrawDebugString(Drawer, FString::Printf(TEXT("%5.2f"), TransformTrajectory.Samples[SampleIdx].TimeInSeconds), Location + FVector(0.0f, 0.0, 10.0f), FRotator::ZeroRotator, LineStyle);

		if (SampleIdx > 0)
		{
			const FVector PrevLocation = TransformTrajectory.Samples[SampleIdx - 1].Position + FVector(0.0f, 0.0f, VerticalOffset);
			DrawDebugLine(Drawer, Location, PrevLocation, LineStyle, bDepthTest);
		}
	}
}

void UDrawDebugLibrary::DrawDebugPose(const FDebugDrawer& Drawer, const TArray<FVector>& BoneLocations, const TArray<FVector>& BoneLinearVelocities, const FTransform& RelativeTransform, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float DrawVelocityLineScale)
{
	DrawDebugPoseFromArrayViews(Drawer, BoneLocations, BoneLinearVelocities, RelativeTransform, LineStyle, bDepthTest, DrawVelocityLineScale);
}

void UDrawDebugLibrary::DrawDebugPoseFromArrayViews(const FDebugDrawer& Drawer, const TArrayView<const FVector> BoneLocations, const TArrayView<const FVector> BoneLinearVelocities, const FTransform& RelativeTransform, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float DrawVelocityLineScale)
{
	const int32 BoneNum = FMath::Min(BoneLocations.Num(), BoneLinearVelocities.Num());

	for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
	{
		const FVector Location = RelativeTransform.TransformPosition(BoneLocations[BoneIdx]);
		const FVector Velocity = RelativeTransform.TransformVector(BoneLinearVelocities[BoneIdx]);

		DrawDebugSphere(Drawer, Location, FRotator::ZeroRotator, LineStyle, bDepthTest);
		DrawDebugLine(Drawer, Location, Location + DrawVelocityLineScale * Velocity, LineStyle, bDepthTest);
	}
}

void UDrawDebugLibrary::DrawDebugCurvesFromSkeletalMeshComponent(
	const FDebugDrawer& Drawer,
	const USkeletalMeshComponent* SkeletalMeshComponent,
	const FVector& Location,
	const FRotator& Rotation,
	const FDrawDebugLineStyle& LineStyle,
	const bool bDepthTest,
	const FDrawDebugStringSettings& Settings)
{
	if (!SkeletalMeshComponent) { return; }

	const FBlendedHeapCurve& Curves = SkeletalMeshComponent->GetAnimCurves();

	FString CurveString;

	Curves.ForEachElement([&CurveString](const UE::Anim::FCurveElement& InCurveElement)
	{
		CurveString = CurveString + FString::Printf(TEXT("%s: % 6.3f\n"), *InCurveElement.Name.ToString(), InCurveElement.Value);
	});

	DrawDebugStringView(
		Drawer,
		CurveString,
		Location,
		Rotation,
		LineStyle,
		bDepthTest,
		Settings);
}

void UDrawDebugLibrary::DrawDebugCurvesToCanvasFromSkeletalMeshComponent(
	const FDebugDrawer& Drawer,
	const USkeletalMeshComponent* SkeletalMeshComponent,
	const FVector2D& Location,
	const FLinearColor Color,
	const FDrawDebugCanvasTextSettings& Settings)
{
	if (!SkeletalMeshComponent) { return; }

	const FBlendedHeapCurve& Curves = SkeletalMeshComponent->GetAnimCurves();

	FString CurveString;

	Curves.ForEachElement([&CurveString](const UE::Anim::FCurveElement& InCurveElement)
		{
			CurveString = CurveString + FString::Printf(TEXT("%s: % 6.3f\n"), *InCurveElement.Name.ToString(), InCurveElement.Value);
		});

	DrawDebugCanvasStringView(
		Drawer,
		Location,
		CurveString,
		Color,
		Settings);
}

void UDrawDebugLibrary::DrawDebugStaticMeshBoundingBox(const FDebugDrawer& Drawer, const UStaticMesh* StaticMesh, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest)
{
	if (StaticMesh)
	{
		DrawDebugBoxSphereBounds(Drawer, Location, Rotation, LineStyle, bDepthTest, StaticMesh->GetBounds());
	}
}

void UDrawDebugLibrary::DrawDebugVelocities(const FDebugDrawer& Drawer, const TArray<FVector>& Locations, const TArray<FVector>& Velocities, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float DrawVelocityLineScale)
{
	DrawDebugVelocitiesArrayView(Drawer, Locations, Velocities, LineStyle, bDepthTest, DrawVelocityLineScale);
}

void UDrawDebugLibrary::DrawDebugVelocitiesArrayView(const FDebugDrawer& Drawer, const TArrayView<const FVector> Locations, const TArrayView<const FVector> Velocities, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const float DrawVelocityLineScale)
{
	const int32 VelocityNum = FMath::Min(Locations.Num(), Velocities.Num());

	for (int32 VelocityIdx = 0; VelocityIdx < VelocityNum; VelocityIdx++)
	{
		DrawDebugVelocity(Drawer, Locations[VelocityIdx], Velocities[VelocityIdx], LineStyle, bDepthTest, DrawVelocityLineScale);
	}
}


void UDrawDebugLibrary::DrawDebugTransformsArrayView(
	const FDebugDrawer& Drawer,
	const TArrayView<const FVector> Locations,
	const TArrayView<const FQuat4f> Rotations,
	const TArrayView<const FVector3f> Scales,
	const FDrawDebugLineStyle& LineStyle,
	const bool bDepthTest,
	const float DrawRadius)
{
	check(Locations.Num() == Rotations.Num());
	check(Locations.Num() == Scales.Num());

	const int32 TransformNum = Locations.Num();
	for (int32 TransformIdx = 0; TransformIdx < TransformNum; TransformIdx++)
	{
		const FTransform Transform = FTransform(
			((FQuat)Rotations[TransformIdx]).GetNormalized(),
			Locations[TransformIdx],
			(FVector)Scales[TransformIdx]);

		DrawDebugTransform(
			Drawer,
			Transform,
			LineStyle,
			bDepthTest,
			DrawRadius);
	}
}

void UDrawDebugLibrary::DrawDebugRotationsQuatArrayView(
	const FDebugDrawer& Drawer,
	const TArrayView<const FVector> Locations,
	const TArrayView<const FQuat4f> Rotations,
	const FDrawDebugLineStyle& LineStyle,
	const bool bDepthTest,
	const float DrawRadius)
{
	const int32 Num = FMath::Min(Locations.Num(), Rotations.Num());

	for (int32 Idx = 0; Idx < Num; Idx++)
	{
		DrawDebugRotation(
			Drawer,
			Locations[Idx],
			(FRotator)Rotations[Idx].Rotator(),
			LineStyle,
			bDepthTest,
			DrawRadius);
	}
}

FLinearColor UDrawDebugLibrary::GetDefaultBoneColor()
{
	return FLinearColor(0.0f, 0.0f, 0.025f, 1.0f);
}

float UDrawDebugLibrary::GetDefaultBoneRadius()
{
	return 1.0f;
}

void UDrawDebugLibrary::DrawDebugBone(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FLinearColor Color, const bool bDepthTest, const float Radius, const int32 Segments, const bool bDrawTransform)
{
	FDrawDebugLineStyle LineStyle;
	LineStyle.Color = Color;

	DrawDebugSimpleSphere(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius, Segments);
	if (bDrawTransform)
	{
		DrawDebugRotation(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius);
	}
}

void UDrawDebugLibrary::DrawDebugBoneLink(const FDebugDrawer& Drawer, const FVector& ChildLocation, const FVector& ParentLocation, const FRotator& ParentRotation, const FLinearColor Color, const bool bDepthTest, const float Radius)
{
	FDrawDebugLineStyle LineStyle;
	LineStyle.Color = Color;

	const FVector HalfWayLocation = (ChildLocation - ParentLocation) / 2.0f + ParentLocation;
	const float Length = (ChildLocation - ParentLocation).Length() - 2.0f * Radius;

	if (Length > 0.0f)
	{
		// Does this need to be changed to be oriented using the X forward direction to match Persona?
		const FVector LocalPosition = ParentRotation.UnrotateVector(ChildLocation - ParentLocation);
		const FQuat LocalRotation = ParentRotation.Quaternion() * FQuat::FindBetween(FVector::UpVector, LocalPosition);

		DrawDebugSquareBasePyramid(Drawer, HalfWayLocation, LocalRotation.Rotator(), LineStyle, bDepthTest, Length, Radius);
	}
}

void UDrawDebugLibrary::DrawDebugSkeletonFromSkinnedMeshComponent(
	const FDebugDrawer& Drawer,
	const USkinnedMeshComponent* SkinnedMeshComponent,
	const FLinearColor Color,
	const bool bDepthTest,
	const FDrawDebugSkeletonSettings& Settings)
{
	if (!SkinnedMeshComponent) { return; }

	const int32 BoneNum = SkinnedMeshComponent->GetNumBones();
	
	const FTransform RootTransform = SkinnedMeshComponent->GetComponentTransform();

	if (Settings.bDrawSimpleSkeleton)
	{
		FDrawDebugLineStyle LineStyle;
		LineStyle.Color = Color;
		LineStyle.Thickness = Settings.BoneRadius;

		TArray<TPair<FVector, FVector>> LineSegments;
		LineSegments.Reserve(BoneNum);

		for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
		{
			const FName BoneName = SkinnedMeshComponent->GetBoneName(BoneIdx);
			const FVector BoneLocation = SkinnedMeshComponent->GetBoneLocation(BoneName, EBoneSpaces::WorldSpace);

			const FName ParentName = SkinnedMeshComponent->GetParentBone(BoneName);
			if (ParentName != NAME_None)
			{
				const FVector ParentLocation = SkinnedMeshComponent->GetBoneLocation(ParentName, EBoneSpaces::WorldSpace);

				LineSegments.Add({ BoneLocation, ParentLocation });
			}
			else if (Settings.bDrawRoot && !(RootTransform.GetLocation() - BoneLocation).IsNearlyZero())
			{
				DrawDebugLine(Drawer, BoneLocation, RootTransform.GetLocation(), LineStyle, bDepthTest);
			}
		}

		DrawDebugLinesPairsArrayView(Drawer, LineSegments, LineStyle, bDepthTest);
	}
	else
	{
		for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
		{
			const FName BoneName = SkinnedMeshComponent->GetBoneName(BoneIdx);
			const FTransform BoneTransform = SkinnedMeshComponent->GetBoneTransform(BoneName, ERelativeTransformSpace::RTS_World);

			DrawDebugBone(Drawer, BoneTransform.GetLocation(), BoneTransform.GetRotation().Rotator(), Color, bDepthTest, Settings.BoneRadius, Settings.BoneSegmentNum, Settings.bDrawTransforms);

			const FName ParentName = SkinnedMeshComponent->GetParentBone(BoneName);
			if (ParentName != NAME_None)
			{
				const FTransform ParentTransform = SkinnedMeshComponent->GetBoneTransform(ParentName, ERelativeTransformSpace::RTS_World);

				DrawDebugBoneLink(Drawer, BoneTransform.GetLocation(), ParentTransform.GetLocation(), ParentTransform.GetRotation().Rotator(), Color, bDepthTest, Settings.BoneRadius);
			}
			else if (Settings.bDrawRoot && !(RootTransform.GetLocation() - BoneTransform.GetLocation()).IsNearlyZero())
			{
				DrawDebugBone(Drawer, RootTransform.GetLocation(), RootTransform.GetRotation().Rotator(), Settings.RootBoneColor, bDepthTest, Settings.BoneRadius, Settings.BoneSegmentNum, Settings.bDrawTransforms);
				DrawDebugBoneLink(Drawer, BoneTransform.GetLocation(), RootTransform.GetLocation(), RootTransform.GetRotation().Rotator(), Settings.RootBoneColor, bDepthTest, Settings.BoneRadius);
			}
		}
	}
}

void UDrawDebugLibrary::DrawDebugSkeletonArrayView(
	const FDebugDrawer& Drawer,
	const FVector RootLocation,
	const FQuat4f RootRotation,
	const TArrayView<const FVector> BoneLocations,
	const TArrayView<const FQuat4f> BoneRotations,
	const TArrayView<const int32> BoneIndices,
	const TArrayView<const int32> BoneParents,
	const FLinearColor Color,
	const bool bDepthTest,
	const FDrawDebugSkeletonSettings& Settings)
{
	if (Settings.bDrawSimpleSkeleton)
	{
		FDrawDebugLineStyle LineStyle;
		LineStyle.Color = Color;
		LineStyle.Thickness = Settings.BoneRadius;

		TArray<TPair<FVector, FVector>> LineSegments;
		LineSegments.Reserve(BoneIndices.Num());

		for (int32 BoneIdx : BoneIndices)
		{
			const int32 ParentIdx = BoneParents[BoneIdx];

			if (ParentIdx != INDEX_NONE)
			{
				LineSegments.Add({ BoneLocations[BoneIdx], BoneLocations[ParentIdx] });
			}
			else if (Settings.bDrawRoot && !(RootLocation - BoneLocations[BoneIdx]).IsNearlyZero())
			{
				DrawDebugLine(Drawer, BoneLocations[BoneIdx], RootLocation, LineStyle, bDepthTest);
			}
		}

		DrawDebugLinesPairsArrayView(Drawer, LineSegments, LineStyle, bDepthTest);
	}
	else
	{

		for (int32 BoneIdx : BoneIndices)
		{
			DrawDebugBone(Drawer, BoneLocations[BoneIdx], (FRotator)BoneRotations[BoneIdx].Rotator(), Color, bDepthTest, Settings.BoneRadius, Settings.BoneSegmentNum, Settings.bDrawTransforms);

			const int32 ParentIdx = BoneParents[BoneIdx];

			if (ParentIdx != INDEX_NONE)
			{
				DrawDebugBoneLink(Drawer, BoneLocations[BoneIdx], BoneLocations[ParentIdx], (FRotator)BoneRotations[ParentIdx].Rotator(), Color, bDepthTest, Settings.BoneRadius);
			}
			else if (Settings.bDrawRoot && !(RootLocation - BoneLocations[BoneIdx]).IsNearlyZero())
			{
				DrawDebugBone(Drawer, RootLocation, (FRotator)RootRotation.Rotator(), Settings.RootBoneColor, bDepthTest, Settings.BoneRadius, Settings.BoneSegmentNum, Settings.bDrawTransforms);
				DrawDebugBoneLink(Drawer, BoneLocations[BoneIdx], RootLocation, (FRotator)RootRotation.Rotator(), Settings.RootBoneColor, bDepthTest, Settings.BoneRadius);
			}
		}
	}
}

void UDrawDebugLibrary::DrawDebugBoneVelocitiesArrayView(
	const FDebugDrawer& Drawer,
	const TArrayView<const FVector> BoneLocations,
	const TArrayView<const FVector3f> BoneVelocities,
	const TArrayView<const int32> BoneIndices,
	const FLinearColor Color,
	const float Thickness,
	const bool bDepthTest,
	const float Scale)
{
	FDrawDebugLineStyle LineStyle;
	LineStyle.Color = Color;
	LineStyle.Thickness = Thickness;

	TArray<TPair<FVector, FVector>> LineSegments;
	LineSegments.Reserve(BoneIndices.Num());

	for (const int32 BoneIdx : BoneIndices)
	{
		LineSegments.Add({ BoneLocations[BoneIdx], BoneLocations[BoneIdx] + Scale * (FVector)BoneVelocities[BoneIdx] });
	}

	DrawDebugLinesPairsArrayView(Drawer, LineSegments, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugRangeTrajectoryArrayView(
	const FDebugDrawer& Drawer,
	const TArrayView<const FVector> TrajectoryLocations,
	const TArrayView<const FQuat4f> TrajectoryRotations,
	const bool bDepthTest,
	const FVector& ForwardVector,
	const bool bDrawOrientations,
	const bool bDrawStartingAtOrigin,
	const FVector& OriginOffset)
{
	const int32 FrameNum = TrajectoryLocations.Num();
	if (FrameNum == 0) { return; }

	FDrawDebugPointStyle PointStyle;
	PointStyle.Color = FColor::Black.WithAlpha(64);
	PointStyle.Thickness = 1.25f;

	FDrawDebugPointStyle ThickPointStyle;
	ThickPointStyle.Color = FColor::Black.WithAlpha(64);
	ThickPointStyle.Thickness = 2.5f;

	FDrawDebugLineStyle LineStyle;
	LineStyle.Color = FColor::Black.WithAlpha(64);
	LineStyle.Thickness = 1.0f;

	TArray<TPair<FVector, FVector>> LineSegments;
	LineSegments.Reserve(FrameNum);

	TArray<FVector> Points;
	Points.Reserve(FrameNum);

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		FVector CurrLocation = TrajectoryLocations[FrameIdx];
		if (bDrawStartingAtOrigin)
		{
			CurrLocation = OriginOffset + ((FQuat)TrajectoryRotations[0]).GetNormalized().UnrotateVector(CurrLocation - TrajectoryLocations[0]);
		}

		const bool bFirstOrLastPoint = FrameIdx == 0 || FrameIdx == FrameNum - 1;

		if (bFirstOrLastPoint)
		{
			DrawDebugPoint(Drawer, CurrLocation, ThickPointStyle, bDepthTest);
		}
		else
		{
			Points.Add(CurrLocation);
		}

		if (bDrawOrientations)
		{
			if (bFirstOrLastPoint || (FrameIdx % 3 == 0))
			{
				FVector CurrDirection = ((FQuat)TrajectoryRotations[FrameIdx]).GetNormalized().RotateVector(ForwardVector);
				if (bDrawStartingAtOrigin)
				{
					CurrDirection = ((FQuat)TrajectoryRotations[0]).GetNormalized().UnrotateVector(CurrDirection);
				}

				DrawDebugFlatArrow(
					Drawer,
					CurrLocation,
					CurrDirection.Rotation(),
					LineStyle,
					bDepthTest,
					15.0f, 15.0f);
			}
		}

		if (FrameIdx < FrameNum - 1)
		{
			FVector NextLocation = TrajectoryLocations[FrameIdx + 1];
			if (bDrawStartingAtOrigin)
			{
				NextLocation = OriginOffset + ((FQuat)TrajectoryRotations[0]).GetNormalized().UnrotateVector(NextLocation - TrajectoryLocations[0]);
			}

			LineSegments.Add({ CurrLocation, NextLocation });
		}
	}

	DrawDebugPointsArrayView(Drawer, Points, PointStyle, bDepthTest);
	DrawDebugLinesPairsArrayView(Drawer, LineSegments, LineStyle, bDepthTest);
}

int32 UDrawDebugLibrary::DrawDebugStringSegmentNum(const FString& String, const FDrawDebugStringSettings& Settings)
{
	return DrawDebugStringViewSegmentNum(String, Settings);
}

int32 UDrawDebugLibrary::DrawDebugStringViewSegmentNum(const FStringView String, const FDrawDebugStringSettings& Settings)
{
	const int32 CharNum = String.Len();

	int32 SegmentNum = 0;

	for (int32 CharIdx = 0; CharIdx < CharNum; CharIdx++)
	{
		if (String.GetData()[CharIdx] >= '!' && String.GetData()[CharIdx] <= '~')
		{
			const int32 TableIdx = String.GetData()[CharIdx] - '!';
			SegmentNum += Settings.bMonospaced ? UE::DrawDebugLibrary::Private::MonoStringLineNums[TableIdx] : UE::DrawDebugLibrary::Private::StringLineNums[TableIdx];
		}
	}

	return SegmentNum;
}

int32 UDrawDebugLibrary::DrawDebugTextSegmentNum(const FText& Text, const FDrawDebugStringSettings& Settings)
{
	return DrawDebugStringViewSegmentNum(Text.ToString(), Settings);
}

int32 UDrawDebugLibrary::DrawDebugNameSegmentNum(const FName& Name, const FDrawDebugStringSettings& Settings)
{
	return DrawDebugStringViewSegmentNum(Name.ToString(), Settings);
}

FVector UDrawDebugLibrary::DrawDebugStringDimensions(const FString& String, const FDrawDebugStringSettings& Settings)
{
	return DrawDebugStringViewDimensions(String, Settings);
}

FVector UDrawDebugLibrary::DrawDebugStringViewDimensions(const FStringView String, const FDrawDebugStringSettings& Settings)
{
	const int32 CharNum = String.Len();

	float XOffsetMax = 0.0f;
	float XOffset = 0.0;
	float YOffset = 0.0;

	for (int32 CharIdx = 0; CharIdx < CharNum; CharIdx++)
	{
		if (String.GetData()[CharIdx] == '\n')
		{
			XOffsetMax = FMath::Max(XOffsetMax, XOffset);
			XOffset = 0.0;
			YOffset -= Settings.HeightScale * Settings.LineSpacing * Settings.Height;
		}
		else if (String.GetData()[CharIdx] == '\r')
		{
			XOffsetMax = FMath::Max(XOffsetMax, XOffset);
			XOffset = 0.0;
		}
		else if (String.GetData()[CharIdx] == '\t')
		{
			XOffset += 4.0f * Settings.WidthScale * 0.5f * Settings.Height + Settings.CharacterSpacing;
		}
		else if (String.GetData()[CharIdx] < '!' || String.GetData()[CharIdx] > '~')
		{
			XOffset += Settings.WidthScale * 0.5f * Settings.Height + Settings.CharacterSpacing;
		}
		else
		{
			const int32 TableIdx = String.GetData()[CharIdx] - '!';
			XOffset += Settings.WidthScale * (Settings.bMonospaced ? 0.5f : (float)UE::DrawDebugLibrary::Private::StringLineAdvs[TableIdx] / 128) * Settings.Height + Settings.CharacterSpacing;
		}
	}

	XOffsetMax = FMath::Max(XOffsetMax, XOffset);
	YOffset -= Settings.HeightScale * Settings.LineSpacing * Settings.Height;

	return FVector(0, XOffsetMax, -YOffset);
}

FVector UDrawDebugLibrary::DrawDebugTextDimensions(const FText& Text, const FDrawDebugStringSettings& Settings)
{
	return DrawDebugStringViewDimensions(Text.ToString(), Settings);
}

FVector UDrawDebugLibrary::DrawDebugNameDimensions(const FName& Name, const FDrawDebugStringSettings& Settings)
{
	return DrawDebugStringViewDimensions(Name.ToString(), Settings);
}

FVector UDrawDebugLibrary::DrawDebugTextCenteringOffset(const FText& Text, const FDrawDebugStringSettings& Settings)
{
	return DrawDebugStringViewCenteringOffset(Text.ToString(), Settings);
}

FVector UDrawDebugLibrary::DrawDebugStringCenteringOffset(const FString& String, const FDrawDebugStringSettings& Settings)
{
	return DrawDebugStringViewCenteringOffset(String, Settings);
}

FVector UDrawDebugLibrary::DrawDebugStringViewCenteringOffset(const FStringView String, const FDrawDebugStringSettings& Settings)
{
	const FVector Dimensions = DrawDebugStringViewDimensions(String, Settings);
	return FVector(0.0f, -Dimensions.Y / 2, Dimensions.Z / 2);
}

FVector UDrawDebugLibrary::DrawDebugNameCenteringOffset(const FName& Name, const FDrawDebugStringSettings& Settings)
{
	return DrawDebugStringViewCenteringOffset(Name.ToString(), Settings);
}

void UDrawDebugLibrary::DrawDebugString(const FDebugDrawer& Drawer, const FString& String, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const FDrawDebugStringSettings& Settings)
{
	DrawDebugStringView(Drawer, String, Location, Rotation, LineStyle, bDepthTest, Settings);
}

void UDrawDebugLibrary::DrawDebugStringView(const FDebugDrawer& Drawer, const FStringView String, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const FDrawDebugStringSettings& Settings)
{
	const int32 CharNum = String.Len();

	TArray<TPair<FVector, FVector>> LineSegments;
	LineSegments.Reserve(DrawDebugStringViewSegmentNum(String, Settings));

	float InitialXOffset = 0.0f;
	float InitialYOffset = 0.0f;

	if (Settings.bCenterHorizontally || Settings.bCenterVertically)
	{
		const FVector Dimensions = DrawDebugStringViewDimensions(String, Settings);
		if (Settings.bCenterHorizontally) { InitialXOffset -= Dimensions.Y / 2; }
		if (Settings.bCenterVertically) { InitialYOffset += Dimensions.Z / 2; }
	}

	float XOffset = InitialXOffset;
	float YOffset = InitialYOffset;

	for (int32 CharIdx = 0; CharIdx < CharNum; CharIdx++)
	{
		if (String.GetData()[CharIdx] == '\n')
		{
			XOffset = InitialXOffset;
			YOffset -= Settings.LineSpacing * Settings.HeightScale * Settings.Height;
		}
		else if (String.GetData()[CharIdx] == '\r')
		{
			XOffset = InitialXOffset;
		}
		else if (String.GetData()[CharIdx] == '\t')
		{
			XOffset += 4.0f * Settings.WidthScale * 0.5f * Settings.Height + Settings.CharacterSpacing;
		}
		else if (String.GetData()[CharIdx] < '!' || String.GetData()[CharIdx] > '~')
		{
			XOffset += Settings.WidthScale * 0.5f * Settings.Height + Settings.CharacterSpacing;
		}
		else
		{
			const int32 TableIdx = String.GetData()[CharIdx] - '!';
			const int32 SegmentNum = Settings.bMonospaced ? UE::DrawDebugLibrary::Private::MonoStringLineNums[TableIdx] : UE::DrawDebugLibrary::Private::StringLineNums[TableIdx];
			const int32 SegmentOff = Settings.bMonospaced ? UE::DrawDebugLibrary::Private::MonoStringLineOffsets[TableIdx] : UE::DrawDebugLibrary::Private::StringLineOffsets[TableIdx];
			const int* Segments = Settings.bMonospaced ? UE::DrawDebugLibrary::Private::MonoStringLineSegments : UE::DrawDebugLibrary::Private::StringLineSegments;

			for (int32 SegmentIdx = 0; SegmentIdx < SegmentNum; SegmentIdx++)
			{
				const int32 XStartInt = (0x000000FF & (Segments[SegmentOff + SegmentIdx] >> 0));
				const int32 YStartInt = (0x000000FF & (Segments[SegmentOff + SegmentIdx] >> 8));
				const int32 XEndInt = (0x000000FF & (Segments[SegmentOff + SegmentIdx] >> 16));
				const int32 YEndInt = (0x000000FF & (Segments[SegmentOff + SegmentIdx] >> 24));

				const FVector Start = FVector(
					0.0f,
					XOffset + Settings.WidthScale * Settings.Height * (((float)XStartInt) / 128),
					YOffset - Settings.HeightScale * Settings.Height * (1.0f - ((float)YStartInt) / 128));

				const FVector End = FVector(
					0.0f,
					XOffset + Settings.WidthScale * Settings.Height * (((float)XEndInt) / 128),
					YOffset - Settings.HeightScale * Settings.Height * (1.0f - ((float)YEndInt) / 128));

				LineSegments.Add({ Rotation.RotateVector(Start) + Location, Rotation.RotateVector(End) + Location });
			}

			XOffset += Settings.WidthScale * (Settings.bMonospaced ? 0.5f : (float)UE::DrawDebugLibrary::Private::StringLineAdvs[TableIdx] / 128) * Settings.Height + Settings.CharacterSpacing;
		}
	}

	DrawDebugLinesPairsArrayView(Drawer, LineSegments, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugName(const FDebugDrawer& Drawer, const FName& Name, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const FDrawDebugStringSettings& Settings)
{
	DrawDebugStringView(Drawer, Name.ToString(), Location, Rotation, LineStyle, bDepthTest, Settings);
}

void UDrawDebugLibrary::DrawDebugNames(const FDebugDrawer& Drawer, const TArray<FName>& Names, const FVector& Location, const FRotator& Rotation, const FString& Separator, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const FDrawDebugStringSettings& Settings)
{
	DrawDebugNamesArrayView(Drawer, Names, Location, Rotation, Separator, LineStyle, bDepthTest, Settings);
}

void UDrawDebugLibrary::DrawDebugNamesArrayView(const FDebugDrawer& Drawer, const TArrayView<const FName> Names, const FVector& Location, const FRotator& Rotation, const FStringView Separator, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const FDrawDebugStringSettings& Settings)
{
	FString String;
	const int32 NameNum = Names.Num();

	for (int32 NameIdx = 0; NameIdx < NameNum; NameIdx++)
	{
		String += Names[NameIdx].ToString();
		if (NameIdx != NameNum - 1)
		{
			String += Separator;
		}
	}

	DrawDebugStringView(Drawer, String, Location, Rotation, LineStyle, bDepthTest, Settings);
}

void UDrawDebugLibrary::DrawDebugText(const FDebugDrawer& Drawer, const FText& Text, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const FDrawDebugStringSettings& Settings)
{
	DrawDebugStringView(Drawer, Text.ToString(), Location, Rotation, LineStyle, bDepthTest, Settings);
}


void UDrawDebugLibrary::VisualLoggerDrawString(const FDebugDrawer& Drawer, const FString& String, const FVector& Location, const FLinearColor Color)
{
	Drawer.VisualLoggerDrawString(String, Location, Color);
}

void UDrawDebugLibrary::VisualLoggerDrawStringView(const FDebugDrawer& Drawer, const FStringView String, const FVector& Location, const FLinearColor Color)
{
	Drawer.VisualLoggerDrawString(String, Location, Color);
}

void UDrawDebugLibrary::VisualLoggerDrawName(const FDebugDrawer& Drawer, const FName& Name, const FVector& Location, const FLinearColor Color)
{
	Drawer.VisualLoggerDrawString(Name.ToString(), Location, Color);
}

void UDrawDebugLibrary::VisualLoggerDrawText(const FDebugDrawer& Drawer, const FText& Text, const FVector& Location, const FLinearColor Color)
{
	Drawer.VisualLoggerDrawString(Text.ToString(), Location, Color);
}


void UDrawDebugLibrary::DrawDebugMoverOrientation(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle, const bool bDepthTest, const FVector ForwardVector, const float Scale)
{
	DrawDebugRotation(
		Drawer,
		Location,
		Rotation,
		LineStyle,
		true,
		Scale / 3.0f);

	const float ArrowSize = (2.0f / 3.0f) * Scale;
	const FVector ArrowDirection = Rotation.RotateVector(ForwardVector);
	const FVector ArrowPosition = Location - (2.0f * ArrowSize / 3.0f) * ArrowDirection;

	DrawDebugFlatArrow(
		Drawer,
		ArrowPosition,
		ArrowDirection.Rotation(),
		LineStyle,
		true,
		2.0f * ArrowSize,
		2.0f * ArrowSize);

	DrawDebugCircle(
		Drawer,
		Location,
		FRotator::ZeroRotator,
		LineStyle,
		true,
		Scale,
		31);
}

void UDrawDebugLibrary::DrawDebugGraph(
	const FDebugDrawer& Drawer,
	const FVector& Location,
	const FRotator& Rotation,
	const TArray<float>& Xvalues,
	const TArray<float>& Yvalues,
	const float Xmin,
	const float Xmax,
	const float Ymin,
	const float Ymax,
	const float XaxisLength,
	const float YaxisLength,
	const FDrawDebugLineStyle& TextLineStyle,
	const FDrawDebugLineStyle& AxesLineStyle,
	const FDrawDebugLineStyle& PlotLineStyle,
	const bool bDepthTest,
	const FDrawDebugGraphAxesSettings& AxesSettings)
{
	DrawDebugGraphArrayView(
		Drawer,
		Location,
		Rotation,
		Xvalues,
		Yvalues,
		Xmin,
		Xmax,
		Ymin,
		Ymax,
		XaxisLength,
		YaxisLength,
		TextLineStyle,
		AxesLineStyle,
		PlotLineStyle,
		bDepthTest,
		AxesSettings);
}

void UDrawDebugLibrary::DrawDebugGraphArrayView(
	const FDebugDrawer& Drawer,
	const FVector& Location,
	const FRotator& Rotation,
	const TArrayView<const float> Xvalues,
	const TArrayView<const float> Yvalues,
	const float Xmin,
	const float Xmax,
	const float Ymin,
	const float Ymax,
	const float XaxisLength,
	const float YaxisLength,
	const FDrawDebugLineStyle& TextLineStyle,
	const FDrawDebugLineStyle& AxesLineStyle,
	const FDrawDebugLineStyle& PlotLineStyle,
	const bool bDepthTest,
	const FDrawDebugGraphAxesSettings& AxesSettings)
{
	DrawDebugGraphAxes(
		Drawer,
		Location,
		Rotation,
		XaxisLength,
		YaxisLength,
		TextLineStyle,
		AxesLineStyle,
		bDepthTest,
		AxesSettings);

	DrawDebugGraphLineArrayView(
		Drawer,
		Location,
		Rotation,
		Xvalues,
		Yvalues,
		Xmin,
		Xmax,
		Ymin,
		Ymax,
		XaxisLength,
		YaxisLength,
		PlotLineStyle,
		bDepthTest);
}

void UDrawDebugLibrary::DrawDebugGraphAxesLabels(
	const FDebugDrawer& Drawer,
	const FVector& Location,
	const FRotator& Rotation,
	const float XaxisLength,
	const float YaxisLength,
	const FDrawDebugLineStyle& TextLineStyle,
	const bool bDepthTest,
	const FDrawDebugGraphAxesSettings& AxesSettings)
{
	const FVector TitleDimension = DrawDebugStringViewDimensions(AxesSettings.Title, AxesSettings.TitleSettings);

	DrawDebugStringView(
		Drawer,
		AxesSettings.Title,
		Location + Rotation.RotateVector(FVector(0.0, XaxisLength / 2.0f - TitleDimension.Y / 2.0f, YaxisLength + 1.1f * TitleDimension.Z)),
		Rotation,
		TextLineStyle,
		bDepthTest,
		AxesSettings.TitleSettings);

	const FVector XlabelDimension = DrawDebugStringViewDimensions(AxesSettings.XaxisLabel, AxesSettings.AxisLabelSettings);

	DrawDebugStringView(
		Drawer,
		AxesSettings.XaxisLabel,
		Location + Rotation.RotateVector(FVector(0.0, XaxisLength / 2.0f - XlabelDimension.Y / 2.0f, -0.1f * XlabelDimension.Z)),
		Rotation,
		TextLineStyle,
		bDepthTest,
		AxesSettings.AxisLabelSettings);

	const FVector YlabelDimension = DrawDebugStringViewDimensions(AxesSettings.YaxisLabel, AxesSettings.AxisLabelSettings);

	DrawDebugStringView(
		Drawer,
		AxesSettings.YaxisLabel,
		Location + Rotation.RotateVector(FVector(0.0, -YlabelDimension.Z - 0.1f * YlabelDimension.Z, YaxisLength / 2 - YlabelDimension.Y / 2)),
		(Rotation.Quaternion() * FQuat::MakeFromRotationVector(FVector(UE_PI / 2, 0, 0))).Rotator(),
		TextLineStyle,
		bDepthTest,
		AxesSettings.AxisLabelSettings);
}

void UDrawDebugLibrary::DrawDebugGraphAxes(
	const FDebugDrawer& Drawer,
	const FVector& Location,
	const FRotator& Rotation,
	const float XaxisLength,
	const float YaxisLength,
	const FDrawDebugLineStyle& TextLineStyle,
	const FDrawDebugLineStyle& AxesLineStyle,
	const bool bDepthTest,
	const FDrawDebugGraphAxesSettings& AxesSettings)
{
	if (AxesSettings.bDrawAxesBox)
	{
		const FVector A0 = Location;
		const FVector A1 = Location + Rotation.RotateVector(FVector(0, XaxisLength, 0));
		const FVector A2 = Location + Rotation.RotateVector(FVector(0, XaxisLength, YaxisLength));
		const FVector A3 = Location + Rotation.RotateVector(FVector(0, 0, YaxisLength));

		DrawDebugLinesPairsArrayView(
			Drawer,
			{
				{ A0, A1 },
				{ A1, A2 },
				{ A2, A3 },
				{ A3, A0 },
			},
			AxesLineStyle,
			bDepthTest);
	}
	else
	{
		DrawDebugOrientedArrow(
			Drawer,
			Location,
			(Rotation.Quaternion() *
				FQuat::MakeFromRotationVector(FVector(0, 0, UE_PI / 2)) *
				FQuat::MakeFromRotationVector(FVector(-UE_PI / 2, 0, 0))).Rotator(),
			XaxisLength,
			AxesLineStyle,
			bDepthTest,
			AxesSettings.AxisArrowSettings);

		DrawDebugOrientedArrow(
			Drawer,
			Location,
			(Rotation.Quaternion() * FQuat::MakeFromRotationVector(FVector(0, -UE_PI / 2, 0))).Rotator(),
			YaxisLength,
			AxesLineStyle,
			bDepthTest,
			AxesSettings.AxisArrowSettings);
	}

	DrawDebugGraphAxesLabels(
		Drawer,
		Location,
		Rotation,
		XaxisLength,
		YaxisLength,
		TextLineStyle,
		bDepthTest,
		AxesSettings);
}

void UDrawDebugLibrary::DrawDebugGraphLine(
	const FDebugDrawer& Drawer,
	const FVector& Location,
	const FRotator& Rotation,
	const TArray<float>& Xvalues,
	const TArray<float>& Yvalues,
	const float Xmin,
	const float Xmax,
	const float Ymin,
	const float Ymax,
	const float XaxisLength,
	const float YaxisLength,
	const FDrawDebugLineStyle& LineStyle,
	const bool bDepthTest)
{
	DrawDebugGraphLineArrayView(Drawer, Location, Rotation, Xvalues, Yvalues, Xmin, Xmax, Ymin, Ymax, XaxisLength, YaxisLength, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugGraphLineArrayView(
	const FDebugDrawer& Drawer,
	const FVector& Location,
	const FRotator& Rotation,
	const TArrayView<const float> Xvalues,
	const TArrayView<const float> Yvalues,
	const float Xmin,
	const float Xmax,
	const float Ymin,
	const float Ymax,
	const float XaxisLength,
	const float YaxisLength,
	const FDrawDebugLineStyle& LineStyle,
	const bool bDepthTest)
{
	const int32 PointNum = FMath::Min(Xvalues.Num(), Yvalues.Num());

	TArray<TPair<FVector, FVector>> LineSegments;
	LineSegments.Reserve(PointNum - 1);

	const float XScale = FMath::Max(Xmax - Xmin, UE_SMALL_NUMBER);
	const float YScale = FMath::Max(Ymax - Ymin, UE_SMALL_NUMBER);

	for (int32 PointIdx = 0; PointIdx < PointNum - 1; PointIdx++)
	{
		const float Xloc0 = XaxisLength * ((Xvalues[PointIdx + 0] - Xmin) / XScale);
		const float Yloc0 = YaxisLength * ((Yvalues[PointIdx + 0] - Ymin) / YScale);
		const float Xloc1 = XaxisLength * ((Xvalues[PointIdx + 1] - Xmin) / XScale);
		const float Yloc1 = YaxisLength * ((Yvalues[PointIdx + 1] - Ymin) / YScale);

		LineSegments.Add({
			Location + Rotation.RotateVector(FVector(0.0, Xloc0, Yloc0)),
			Location + Rotation.RotateVector(FVector(0.0, Xloc1, Yloc1)) });
	}

	DrawDebugLinesPairsArrayView(Drawer, LineSegments, LineStyle, bDepthTest);
}

void UDrawDebugLibrary::DrawDebugGraphLegend(
	const FDebugDrawer& Drawer,
	const FVector& Location,
	const FRotator& Rotation,
	const TArray<FLinearColor>& LegendColors,
	const TArray<FString>& LegendLabels,
	const float XaxisLength,
	const float YaxisLength,
	const FDrawDebugLineStyle& TextLineStyle,
	const FDrawDebugLineStyle& IconLineStyle,
	const float IconSize,
	const bool bDepthTest,
	const FDrawDebugStringSettings& LegendSettings)
{
	DrawDebugGraphLegendArrayView(Drawer, Location, Rotation, LegendColors, LegendLabels, XaxisLength, YaxisLength, TextLineStyle, IconLineStyle, IconSize, bDepthTest, LegendSettings);
}

void UDrawDebugLibrary::DrawDebugGraphLegendArrayView(
	const FDebugDrawer& Drawer,
	const FVector& Location,
	const FRotator& Rotation,
	const TArrayView<const FLinearColor> LegendColors,
	const TArrayView<const FString> LegendLabels,
	const float XaxisLength,
	const float YaxisLength,
	const FDrawDebugLineStyle& TextLineStyle,
	const FDrawDebugLineStyle& IconLineStyle,
	const float IconSize,
	const bool bDepthTest,
	const FDrawDebugStringSettings& LegendSettings)
{
	const int32 LegendNum = FMath::Min(LegendColors.Num(), LegendLabels.Num());

	float TotalHeight = 0.0f;
	for (int32 LegendIdx = 0; LegendIdx < LegendNum; LegendIdx++)
	{
		TotalHeight = DrawDebugStringViewDimensions(LegendLabels[LegendIdx], LegendSettings).Z;
	}

	float VerticalOffset = TotalHeight / 2.0f + YaxisLength / 2.0f;
	for (int32 LegendIdx = 0; LegendIdx < LegendNum; LegendIdx++)
	{
		const float LegendHeight = DrawDebugStringViewDimensions(LegendLabels[LegendIdx], LegendSettings).Z;

		const FQuat IconRotation = FQuat::MakeFromRotationVector(FVector(0.0f, UE_PI / 2.0, 0.0f));

		DrawDebugDiamond(
			Drawer, 
			Location + Rotation.RotateVector(FVector(0.0, XaxisLength + 2.0f * IconSize, VerticalOffset - LegendHeight / 2.0f)),
			(Rotation.Quaternion() * IconRotation).Rotator(),
			DrawDebugLineStyleWithColor(IconLineStyle, LegendColors[LegendIdx]), bDepthTest, IconSize / 2.0f);

		DrawDebugStringView(
			Drawer,
			LegendLabels[LegendIdx],
			Location + Rotation.RotateVector(FVector(0.0, XaxisLength + 4.0f * IconSize, VerticalOffset)),
			Rotation,
			DrawDebugLineStyleWithColor(TextLineStyle, LegendColors[LegendIdx]),
			bDepthTest,
			LegendSettings);

		VerticalOffset += LegendHeight;
	}
}