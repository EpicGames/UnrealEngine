// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_DrawDebugLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_DrawDebugLibrary)

FRigVMFunction_MakeNullDebugDrawer_Execute()
{
	Drawer = UDrawDebugLibrary::MakeNullDebugDrawer();
}

FRigVMFunction_MakeDebugDrawer_Execute()
{
	Drawer = UDrawDebugLibrary::MakeRigVMDebugDrawer(ExecuteContext.GetDrawInterface(), ExecuteContext.GetOwningComponent());
}

FRigVMFunction_MakeVisualLoggerDebugDrawer_Execute()
{
	Drawer = UDrawDebugLibrary::MakeVisualLoggerDebugDrawer(ExecuteContext.GetOwningObject(), Category, Verbosity, false, false);
}

FRigVMFunction_MakeMergedDebugDrawer_Execute()
{
	Drawer = UDrawDebugLibrary::MakeMergedDebugDrawerArrayView(Drawers);
}

FRigUnit_VisualLoggerLogString_Execute()
{
	UDrawDebugLibrary::VisualLoggerLogString(Drawer, String, Color);
}

FRigVMFunction_MakeLinearlySpacedFloatArray_Execute()
{
	UDrawDebugLibrary::MakeLinearlySpacedFloatArray(Values, Start, Stop, Num);
}

FRigUnit_AddToFloatHistoryArray_Execute()
{
	UDrawDebugLibrary::AddToFloatHistoryArray(Values, NewValue, MaxHistoryNum);
}

FRigUnit_AddToVectorHistoryArray_Execute()
{
	UDrawDebugLibrary::AddToVectorHistoryArray(Values, NewValue, MaxHistoryNum);
}

FRigUnit_AddToNameHistoryArray_Execute()
{
	UDrawDebugLibrary::AddToNameHistoryArray(Values, NewValue, MaxHistoryNum);
}

FRigVMFunction_DrawDebugLocalOffset_Execute()
{
	UDrawDebugLibrary::DrawDebugLocalOffset(Location, Rotation, DrawLocation, DrawRotation, DrawOffset);
}

FRigVMFunction_DrawDebugOrientUprightToFloor_Execute()
{
	UDrawDebugLibrary::DrawDebugOrientUprightToFloor(Location, Rotation, DrawWorldLocation, DrawUprightOffset);
}

FRigVMFunction_DrawDebugOrientFloorToUpright_Execute()
{
	UDrawDebugLibrary::DrawDebugOrientFloorToUpright(Location, Rotation, DrawWorldLocation, DrawFloorOffset);
}

FRigVMFunction_DrawDebugOrientUprightToCamera_Execute()
{
	UDrawDebugLibrary::DrawDebugOrientUprightToCamera(Location, Rotation, DrawWorldLocation, DrawUprightOffset, CameraRotation, bUseOnlyYaw);
}

FRigVMFunction_DrawDebugOrientFloorToCamera_Execute()
{
	UDrawDebugLibrary::DrawDebugOrientFloorToCamera(Location, Rotation, DrawWorldLocation, DrawFloorOffset, CameraRotation, bUseOnlyYaw);
}

FRigUnit_DrawDebugPoint_Execute()
{
	UDrawDebugLibrary::DrawDebugPoint(Drawer, Location, PointStyle, bDepthTest);
}

FRigUnit_DrawDebugPoints_Execute()
{
	UDrawDebugLibrary::DrawDebugPointsArrayView(Drawer, Locations, PointStyle, bDepthTest);
}

FRigUnit_DrawDebugLine_Execute()
{
    UDrawDebugLibrary::DrawDebugLine(Drawer, StartLocation, EndLocation, LineStyle, bDepthTest);
}

FRigUnit_DrawDebugLines_Execute()
{
	UDrawDebugLibrary::DrawDebugLinesArrayView(Drawer, StartLocations, EndLocations, LineStyle, bDepthTest);
}

FRigUnit_DrawDebugTriangularBasePyramid_Execute()
{
	UDrawDebugLibrary::DrawDebugTriangularBasePyramid(Drawer, Location, Rotation, LineStyle, bDepthTest, Length, Width);
}

FRigUnit_DrawDebugSquareBasePyramid_Execute()
{
	UDrawDebugLibrary::DrawDebugSquareBasePyramid(Drawer, Location, Rotation, LineStyle, bDepthTest, Length, Width);
}

FRigUnit_DrawDebugCone_Execute()
{
	UDrawDebugLibrary::DrawDebugCone(Drawer, Location, Rotation, LineStyle, bDepthTest, Length, Radius, Segments);
}

FRigUnit_DrawDebugConeLookAt_Execute()
{
	UDrawDebugLibrary::DrawDebugConeLookAt(Drawer, Location, Direction, LineStyle, bDepthTest, Length, Angle, Segments);
}

FRigUnit_DrawDebugArc_Execute()
{
	UDrawDebugLibrary::DrawDebugArc(Drawer, Location, Rotation, Angle, LineStyle, bDepthTest, Radius, Segments);
}

FRigUnit_DrawDebugCircle_Execute()
{
	UDrawDebugLibrary::DrawDebugCircle(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius, Segments);
}

FRigUnit_DrawDebugCircleTick_Execute()
{
	UDrawDebugLibrary::DrawDebugCircleTick(Drawer, Location, Rotation, Angle, LineStyle, bDepthTest, Radius, Length, bInside);
}

FRigUnit_DrawDebugCircleTicks_Execute()
{
	UDrawDebugLibrary::DrawDebugCircleTicksArrayView(Drawer, Location, Rotation, Angles, LineStyle, bDepthTest, Radius, Length, bInside);
}

FRigUnit_DrawDebugCircleOutline_Execute()
{
	UDrawDebugLibrary::DrawDebugCircleOutline(Drawer, Location, Rotation, LineStyle, bDepthTest, InnerRadius, OuterRadius, Segments);
}

FRigUnit_DrawDebugTriangle_Execute()
{
	UDrawDebugLibrary::DrawDebugTriangle(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius);
}

FRigUnit_DrawDebugSquare_Execute()
{
	UDrawDebugLibrary::DrawDebugSquare(Drawer, Location, Rotation, LineStyle, bDepthTest, HalfLength);
}

FRigUnit_DrawDebugCross_Execute()
{
	UDrawDebugLibrary::DrawDebugCross(Drawer, Location, Rotation, LineStyle, bDepthTest, HalfLength);
}

FRigUnit_DrawDebugDiamond_Execute()
{
	UDrawDebugLibrary::DrawDebugDiamond(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius);
}

FRigUnit_DrawDebugPentagon_Execute()
{
	UDrawDebugLibrary::DrawDebugPentagon(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius);
}

FRigUnit_DrawDebugHexagon_Execute()
{
	UDrawDebugLibrary::DrawDebugHexagon(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius);
}

FRigUnit_DrawDebugHeptagon_Execute()
{
	UDrawDebugLibrary::DrawDebugHeptagon(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius);
}

FRigUnit_DrawDebugOctagon_Execute()
{
	UDrawDebugLibrary::DrawDebugOctagon(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius);
}

FRigUnit_DrawDebugNonagon_Execute()
{
	UDrawDebugLibrary::DrawDebugNonagon(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius);
}

FRigUnit_DrawDebugDecagon_Execute()
{
	UDrawDebugLibrary::DrawDebugDecagon(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius);
}

FRigUnit_DrawDebugRegularPolygon_Execute()
{
	UDrawDebugLibrary::DrawDebugRegularPolygon(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius, Sides);
}

FRigUnit_DrawDebugCheckMark_Execute()
{
	UDrawDebugLibrary::DrawDebugCheckMark(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius);
}

FRigUnit_DrawDebugCheckBox_Execute()
{
	UDrawDebugLibrary::DrawDebugCheckBox(Drawer, Location, Rotation, LineStyle, bDepthTest, HalfLength, bChecked, TickRatio);
}

FRigUnit_DrawDebugCrossBox_Execute()
{
	UDrawDebugLibrary::DrawDebugCrossBox(Drawer, Location, Rotation, LineStyle, bDepthTest, HalfLength, bCrossed, CrossRatio);
}

FRigUnit_DrawDebugRadioButton_Execute()
{
	UDrawDebugLibrary::DrawDebugRadioButton(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius, bSelected, InnerButtonRatio, Segments);
}

FRigUnit_DrawDebugLocator_Execute()
{
	UDrawDebugLibrary::DrawDebugLocator(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius);
}

FRigUnit_DrawDebugCrossLocator_Execute()
{
	UDrawDebugLibrary::DrawDebugCrossLocator(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius);
}

FRigUnit_DrawDebugBox_Execute()
{
	UDrawDebugLibrary::DrawDebugBox(Drawer, Location, Rotation, LineStyle, bDepthTest, HalfExtents);
}

FRigUnit_DrawDebugSphere_Execute()
{
	UDrawDebugLibrary::DrawDebugSphere(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius, Segments);
}

FRigUnit_DrawDebugSimpleSphere_Execute()
{
	UDrawDebugLibrary::DrawDebugSimpleSphere(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius, Segments);
}

FRigUnit_DrawDebugCapsule_Execute()
{
	UDrawDebugLibrary::DrawDebugCapsule(Drawer, Location, Rotation, LineStyle, bDepthTest, Radius, HalfLength, Segments);
}

FRigUnit_DrawDebugCapsuleLine_Execute()
{
	UDrawDebugLibrary::DrawDebugCapsuleLine(Drawer, StartLocation, EndLocation, LineStyle, bDepthTest, Radius, Segments);
}

FRigUnit_DrawDebugFrustum_Execute()
{
	UDrawDebugLibrary::DrawDebugFrustum(Drawer, FrustumToWorld, LineStyle, bDepthTest);
}

FRigUnit_DrawDebugArrowHead_Execute()
{
	UDrawDebugLibrary::DrawDebugArrowHead(Drawer, Location, Rotation, LineStyle, bDepthTest, Type, Size);
}

FRigUnit_DrawDebugArrow_Execute()
{
	UDrawDebugLibrary::DrawDebugArrow(Drawer, StartLocation, EndLocation, LineStyle, bDepthTest, Settings);
}

FRigUnit_DrawDebugOrientedArrow_Execute()
{
	UDrawDebugLibrary::DrawDebugOrientedArrow(Drawer, Location, Rotation, Length, LineStyle, bDepthTest, Settings);
}

FRigUnit_DrawDebugGroundTargetArrow_Execute()
{
	UDrawDebugLibrary::DrawDebugGroundTargetArrow(Drawer, Location, LineStyle, bDepthTest, Length, ArrowHeadSize, Radius, Segments);
}

FRigUnit_DrawDebugFlatArrow_Execute()
{
	UDrawDebugLibrary::DrawDebugFlatArrow(Drawer, Location, Rotation, LineStyle, bDepthTest, Length, Width);
}

FRigUnit_DrawDebugCircleArrow_Execute()
{
	UDrawDebugLibrary::DrawDebugCircleArrow(Drawer, Location, Rotation, Angle, LineStyle, bDepthTest, Radius, Length, Settings);
}

FRigUnit_DrawDebugArcArrow_Execute()
{
	UDrawDebugLibrary::DrawDebugArcArrow(Drawer, Location, Rotation, Angle, LineStyle, bDepthTest, Radius, Segments, Settings);
}

FRigUnit_DrawDebugCatmullRomSplineStart_Execute()
{
	UDrawDebugLibrary::DrawDebugCatmullRomSplineStart(Drawer, V0, V1, V2, LineStyle, bDepthTest, bMonotonic, Segments);
}

FRigUnit_DrawDebugCatmullRomSplineEnd_Execute()
{
	UDrawDebugLibrary::DrawDebugCatmullRomSplineEnd(Drawer, V0, V1, V2, LineStyle, bDepthTest, bMonotonic, Segments);
}

FRigUnit_DrawDebugCatmullRomSplineSection_Execute()
{
	UDrawDebugLibrary::DrawDebugCatmullRomSplineSection(Drawer, V0, V1, V2, V3, LineStyle, bDepthTest, bMonotonic, Segments);
}

FRigUnit_DrawDebugCatmullRomSpline_Execute()
{
	UDrawDebugLibrary::DrawDebugCatmullRomSplineArrayView(Drawer, Points, LineStyle, bDepthTest, bMonotonic, Segments);
}

FRigUnit_DrawDebugAngle_Execute()
{
	UDrawDebugLibrary::DrawDebugAngle(Drawer, Location, Rotation, Angle, LineStyle, bDepthTest, LineLength, AngleRadius, Segments);
}

FRigUnit_DrawDebugAngleBetween_Execute()
{
	UDrawDebugLibrary::DrawDebugAngleBetween(Drawer, P0, P1, P2, LineStyle, bDepthTest, AngleRadius, Segments);
}

FRigUnit_DrawDebugLocation_Execute()
{
	UDrawDebugLibrary::DrawDebugLocation(Drawer, Location, LineStyle, bDepthTest, DrawRadius, Segments);
}

FRigUnit_DrawDebugLocations_Execute()
{
	UDrawDebugLibrary::DrawDebugLocationsArrayView(Drawer, Locations, LineStyle, bDepthTest, DrawRadius, Segments);
}

FRigUnit_DrawDebugRotation_Execute()
{
	UDrawDebugLibrary::DrawDebugRotation(Drawer, Location, Rotation, LineStyle, bDepthTest, DrawRadius);
}

FRigUnit_DrawDebugDirection_Execute()
{
	UDrawDebugLibrary::DrawDebugDirection(Drawer, Location, Direction, LineStyle, bDepthTest, DrawArrowLength, ArrowHeadScale);
}

FRigUnit_DrawDebugVelocity_Execute()
{
	UDrawDebugLibrary::DrawDebugVelocity(Drawer, Location, Velocity, LineStyle, bDepthTest, DrawVelocityLineScale);
}

FRigUnit_DrawDebugVelocities_Execute()
{
	UDrawDebugLibrary::DrawDebugVelocitiesArrayView(Drawer, Locations, Velocities, LineStyle, bDepthTest, DrawVelocityLineScale);
}

FRigUnit_DrawDebugTransform_Execute()
{
	UDrawDebugLibrary::DrawDebugTransform(Drawer, Transform, LineStyle, bDepthTest, DrawRadius);
}

FRigUnit_DrawDebugEvent_Execute()
{
	UDrawDebugLibrary::DrawDebugEvent(Drawer, bTimeUntilEventKnown, TimeUntilEvent, Location, LineStyle, bDepthTest, Size);
}

FRigUnit_DrawDebugChair_Execute()
{
	UDrawDebugLibrary::DrawDebugChair(Drawer, Location, Rotation, LineStyle, bDepthTest, Settings);
}

FRigUnit_DrawDebugDoor_Execute()
{
	UDrawDebugLibrary::DrawDebugDoor(Drawer, Location, Rotation, LineStyle, bDepthTest, Settings);
}

FRigUnit_DrawDebugCamera_Execute()
{
	UDrawDebugLibrary::DrawDebugCamera(Drawer, Location, Rotation, LineStyle, bDepthTest, Scale, FOVDegrees);
}

FRigUnit_DrawDebugBackpack_Execute()
{
	UDrawDebugLibrary::DrawDebugBackpack(Drawer, Location, Rotation, LineStyle, bDepthTest, Settings);
}

FRigUnit_DrawDebugBriefcase_Execute()
{
	UDrawDebugLibrary::DrawDebugBriefcase(Drawer, Location, Rotation, LineStyle, bDepthTest, Settings);
}

FRigUnit_DrawDebugTrajectory_Execute()
{
	UDrawDebugLibrary::DrawDebugTrajectoryFromArrayViews(Drawer, Locations, Directions, RelativeTransform, LineStyle, bDepthTest, DrawArrowLength, PointRadius, ArrowHeadScale, Segments, VerticalOffset);
}

FRigUnit_DrawDebugTransformTrajectory_Execute()
{
	UDrawDebugLibrary::DrawDebugTransformTrajectory(Drawer, TransformTrajectory, LineStyle, bDepthTest, Radius, VerticalOffset);
}

FRigUnit_DrawDebugMoverOrientation_Execute()
{
	UDrawDebugLibrary::DrawDebugMoverOrientation(Drawer, Location, Rotation, LineStyle, bDepthTest, ForwardVector, Scale);
}

FRigVMFunction_DrawDebugGetDefaultBoneColor_Execute()
{
	Color = FLinearColor(0.0f, 0.0f, 0.025f, 1.0f);
}

FRigVMFunction_DrawDebugGetDefaultBoneRadius_Execute()
{
	Radius = 1.0f;
}

FRigUnit_DrawDebugBone_Execute()
{
	UDrawDebugLibrary::DrawDebugBone(Drawer, Location, Rotation, Color, bDepthTest, Radius, Segments, bDrawTransform);
}

FRigUnit_DrawDebugBoneLink_Execute()
{
	UDrawDebugLibrary::DrawDebugBoneLink(Drawer, ChildLocation, ParentLocation, ParentRotation, Color, bDepthTest, Radius);
}

FRigUnit_DrawDebugSkeletonFromSkinnedMeshComponent_Execute()
{
	UDrawDebugLibrary::DrawDebugSkeletonFromSkinnedMeshComponent(Drawer, SkinnedMeshComponent.Get(), Color, bDepthTest, Settings);
}

FRigUnit_DrawDebugPose_Execute()
{
	UDrawDebugLibrary::DrawDebugPoseFromArrayViews(Drawer, BoneLocations, BoneLinearVelocities, RelativeTransform, LineStyle, bDepthTest, DrawVelocityLineScale);
}

FRigVMFunction_DrawDebugStringSegmentNum_Execute()
{
	SegmentNum = UDrawDebugLibrary::DrawDebugStringViewSegmentNum(String, Settings);
}

FRigVMFunction_DrawDebugNameSegmentNum_Execute()
{
	SegmentNum = UDrawDebugLibrary::DrawDebugNameSegmentNum(Name, Settings);
}

FRigVMFunction_DrawDebugStringDimensions_Execute()
{
	Dimensions = UDrawDebugLibrary::DrawDebugStringViewDimensions(String, Settings);
}

FRigVMFunction_DrawDebugNameDimensions_Execute()
{
	Dimensions = UDrawDebugLibrary::DrawDebugNameDimensions(Name, Settings);
}

FRigVMFunction_DrawDebugStringCenteringOffset_Execute()
{
	Offset = UDrawDebugLibrary::DrawDebugStringViewCenteringOffset(String, Settings);
}

FRigVMFunction_DrawDebugNameCenteringOffset_Execute()
{
	Offset = UDrawDebugLibrary::DrawDebugNameCenteringOffset(Name, Settings);
}

FRigUnit_DrawDebugString_Execute()
{
	UDrawDebugLibrary::DrawDebugString(Drawer, String, Location, Rotation, LineStyle, bDepthTest, Settings);
}

FRigUnit_DrawDebugName_Execute()
{
	UDrawDebugLibrary::DrawDebugName(Drawer, Name, Location, Rotation, LineStyle, bDepthTest, Settings);
}

FRigUnit_DrawDebugNames_Execute()
{
	UDrawDebugLibrary::DrawDebugNamesArrayView(Drawer, Names, Location, Rotation, Separator, LineStyle, bDepthTest, Settings);
}

FRigUnit_DrawDebugVisualLoggerDrawString_Execute()
{
	UDrawDebugLibrary::VisualLoggerDrawStringView(Drawer, String, Location, Color);
}

FRigUnit_DrawDebugVisualLoggerDrawName_Execute()
{
	UDrawDebugLibrary::VisualLoggerDrawName(Drawer, Name, Location, Color);
}

FRigUnit_DrawDebugGraph_Execute()
{
	UDrawDebugLibrary::DrawDebugGraphArrayView(Drawer, Location, Rotation, Xvalues, Yvalues, Xmin, Xmax, Ymin, Ymax, XaxisLength, YaxisLength, TextLineStyle, AxesLineStyle, PlotLineStyle, bDepthTest, AxesSettings);
}

FRigUnit_DrawDebugGraphAxesLabels_Execute()
{
	UDrawDebugLibrary::DrawDebugGraphAxesLabels(Drawer, Location, Rotation, XaxisLength, YaxisLength, TextLineStyle, bDepthTest, AxesSettings);
}

FRigUnit_DrawDebugGraphAxes_Execute()
{
	UDrawDebugLibrary::DrawDebugGraphAxes(Drawer, Location, Rotation, XaxisLength, YaxisLength, TextLineStyle, AxesLineStyle, bDepthTest, AxesSettings);
}

FRigUnit_DrawDebugGraphLine_Execute()
{
	UDrawDebugLibrary::DrawDebugGraphLineArrayView(Drawer, Location, Rotation, Xvalues, Yvalues, Xmin, Xmax, Ymin, Ymax, XaxisLength, YaxisLength, LineStyle, bDepthTest);
}

FRigUnit_DrawDebugGraphLegend_Execute()
{
	UDrawDebugLibrary::DrawDebugGraphLegendArrayView(Drawer, Location, Rotation, LegendColors, LegendLabels, XaxisLength, YaxisLength, TextLineStyle, IconLineStyle, IconSize, bDepthTest, LegendSettings);
}