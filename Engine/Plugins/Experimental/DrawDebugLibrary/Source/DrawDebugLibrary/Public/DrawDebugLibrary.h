// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "EngineDefines.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "DrawDebugLibrary.generated.h"

#define UE_API DRAWDEBUGLIBRARY_API

class FPrimitiveDrawInterface;
struct FRigVMDrawInterface;
struct FTransformTrajectory;
struct FAnimInstanceProxy;
class USkinnedMeshComponent;
class UStaticMesh;
class UCanvas;
class FCanvas;
class UFont;

class FDrawDebugLibraryModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

/** Blueprint accessible version of log verbosity for the Visual Logger Drawer */
UENUM(BlueprintType, Category = "Draw Debug Library")
enum class EDrawDebugLogVerbosity : uint8
{
	/** Always prints a fatal error to console (and log file) and crashes (even if logging is disabled) */
	Fatal,

	/**
	 * Prints an error to console (and log file).
	 * Commandlets and the editor collect and report errors. Error messages result in commandlet failure.
	 */
	Error,

	/**
	 * Prints a warning to console (and log file).
	 * Commandlets and the editor collect and report warnings. Warnings can be treated as an error.
	 */
	Warning,

	/** Prints a message to console (and log file) */
	Display,

	/** Prints a message to a log file (does not print to console) */
	Log,

	/**
	 * Prints a verbose message to a log file (if Verbose logging is enabled for the given category,
	 * usually used for detailed logging)
	 */
	Verbose,

	/**
	 * Prints a verbose message to a log file (if VeryVerbose logging is enabled,
	 * usually used for detailed logging that would otherwise spam output)
	 */
	VeryVerbose,
};


/** Style for debug drawing points */
USTRUCT(BlueprintType, Category = "Draw Debug Library")
struct FDrawDebugPointStyle
{
	GENERATED_BODY()

	/** Construct a Point Style */
	UE_API FDrawDebugPointStyle(
		const FLinearColor InColor = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f),
		const float InThickness = 0.0f);

	/** Color of the point. Alpha controls the opacity. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f);

	/** Thickness of the point. Set to 0.0 for pixel-sized points. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float Thickness = 0.0f;
};


/** Enum for different line types */
UENUM(BlueprintType, Category = "Draw Debug Library")
enum class EDrawDebugLineType : uint8
{
	/** Normal solid line */
	Solid,

	/** Line made up of multiple dashes */
	Dashed,

	/** Line drawn using points */
	Dotted,
};


/** Style for debug drawing lines */
USTRUCT(BlueprintType, Category = "Draw Debug Library")
struct FDrawDebugLineStyle
{
	GENERATED_BODY()

	/** Construct a Line Style */
	UE_API FDrawDebugLineStyle(
		const FLinearColor InColor = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f),
		const float InThickness = 0.0f,
		const EDrawDebugLineType InLineType = EDrawDebugLineType::Solid,
		const float InDashWidth = 2.5f,
		const float InDashSpacing = 1.0f,
		const float InDotSpacing = 1.0f);

	/** Color of the line. Alpha controls the opacity. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f);

	/** Thickness of the line. Set to 0.0 for pixel-sized lines. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float Thickness = 0.0f;

	/** Line style to draw with */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	EDrawDebugLineType LineType = EDrawDebugLineType::Solid;

	/** Width of the dashes when using dashed style. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", AdvancedDisplay, meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float DashWidth = 2.5f;

	/** Spacing of the dashes when using dashed style. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", AdvancedDisplay, meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float DashSpacing = 1.0f;

	/** Spacing of the dots when using dotted style. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", AdvancedDisplay, meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float DotSpacing = 1.0f;
};


/** Enum for different arrow head types */
UENUM(BlueprintType, Category = "Draw Debug Library")
enum class EDrawDebugArrowHead : uint8
{
	/** Simple pointed arrow */
	Simple,

	/** Four-way arrow head */
	FourWay,

	/** Triangular-based pyramid arrow head */
	TriangularPyramid,

	/** Square-based pyramid arrow head */
	SquarePyramid,

	/** Cone arrow head */
	Cone,

	/** Flat triangle arrow head */
	Triangle,

	/** Flat square arrow head */
	Square,

	/** Flat diamond arrow head */
	Diamond,

	/** Flat circle arrow head */
	Circle,

	/** 3D sphere arrow head */
	Sphere,

	/** 3D box arrow head */
	Box,

	/** No arrow head */
	None,
};


/** Settings for debug drawing arrows */
USTRUCT(BlueprintType, Category = "Draw Debug Library")
struct FDrawDebugArrowSettings
{
	GENERATED_BODY()

	/** If to draw an arrow head on the start of the arrow */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	bool bArrowheadOnStart = false;

	/** If the line should end at the start arrow head */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (EditCondition = "bArrowheadOnStart", HideEditConditionToggle))
	bool bArrowLineEndsAtStartHead = false;

	/** Size of the start arrow head */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (EditCondition = "bArrowheadOnStart", HideEditConditionToggle, ClampMin = "0", UIMin = "0", Units = "cm"))
	float ArrowHeadStartSize = 5.0f;

	/** Type of start arrow head */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (EditCondition = "bArrowheadOnStart", HideEditConditionToggle))
	EDrawDebugArrowHead ArrowHeadStartType = EDrawDebugArrowHead::Simple;

	/** If to draw an arrow head on the end of the arrow */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	bool bArrowheadOnEnd = true;

	/** If the line should end at the end arrow head */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (EditCondition = "bArrowheadOnEnd", HideEditConditionToggle))
	bool bArrowLineEndsAtEndHead = false;

	/** Size of the end arrow head */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (EditCondition = "bArrowheadOnEnd", HideEditConditionToggle, ClampMin = "0", UIMin = "0", Units = "cm"))
	float ArrowHeadEndSize = 5.0f;

	/** Type of end arrow head */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (EditCondition = "bArrowheadOnEnd", HideEditConditionToggle))
	EDrawDebugArrowHead ArrowHeadEndType = EDrawDebugArrowHead::Simple;
};

/** Settings for debug drawing chairs */
USTRUCT(BlueprintType, Category = "Draw Debug Library")
struct FDrawDebugChairSettings
{
	GENERATED_BODY()

	/** Width of the chair */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float Width = 40.0f;

	/** Height of the seat of the chair off the ground */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float SeatHeight = 40.0f;

	/** Height of the back of the chair off the seat */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float BackHeight = 40.0f;

	/** Tilt of the chair backwards in cm */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (Units = "cm"))
	float BackTilt = 10.0f;
};

/** Settings for debug drawing backpacks */
USTRUCT(BlueprintType, Category = "Draw Debug Library")
struct FDrawDebugBackpackSettings
{
	GENERATED_BODY()

	/** Width of the backpack */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float Width = 30.0f;

	/** Height of the backpack */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float Height = 45.0f;

	/** Depth of the backpack */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float Depth = 15.0f;

	/** Distance the straps extend outward */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float StrapDistance = 25.0f;

	/** The height at which the straps are in contact */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (Units = "cm"))
	float StrapContactHeight = 25.0f;

	/** Width of the straps */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (Units = "cm"))
	float StrapWidth = 2.5f;

	/** Inset of the straps horizontally */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (Units = "cm"))
	float StrapHorizontalInset = 5.0f;

	/** Inset of the straps vertically */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (Units = "cm"))
	float StrapVerticalInset = 2.5f;
};

/** Settings for debug drawing briefcases */
USTRUCT(BlueprintType, Category = "Draw Debug Library")
struct FDrawDebugBriefcaseSettings
{
	GENERATED_BODY()

	/** Half-extents of the briefcase */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	FVector HalfExtents = FVector(20.0f, 5.0f, 15.0f);

	/** Length of the handle */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float HandleLength = 10.0f;

	/** The height of the handle above the briefcase */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float HandleHeight = 5.0f;

	/** Width of the handle */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float HandleWidth = 2.5f;
};

/** Settings for debug drawing doors */
USTRUCT(BlueprintType, Category = "Draw Debug Library")
struct FDrawDebugDoorSettings
{
	GENERATED_BODY()

	/** Width of the door */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float Width = 100.0f;

	/** Height of the door */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float Height = 200.0f;

	/** How much the door is inset from the frame */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float Inset = 5.0f;

	/** The height of the handle off the ground */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float HandleHeight = 90.0f;

	/** The offset of the handle into the door */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float HandleOffset = 20.0f;

	/** The radius of the handle */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float HandleRadius = 3.0f;

	/** How far the handle extends outward from the door */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float HandleExtension = 2.0f;

	/** If to place a handle on the outside entrance of the door */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	bool bOuterHandle = true;

	/** If to place a handle on the inside entrance of the door */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	bool bInnerHandle = true;

	/** If to place the handle on the left-hand-side or the right-hand-side of the door from the perspective of entering */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	bool bHandleOnLeft = true;

	/** If to draw an arrow on the floor showing the natural door entry direction */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	bool bDrawEntryArrow = true;

	/** How open the door is in degrees */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (Units = "deg"))
	float OpenAngle = 40.0f;
};

/** Settings for debug drawing skeletons */
USTRUCT(BlueprintType, Category = "Draw Debug Library")
struct FDrawDebugSkeletonSettings
{
	GENERATED_BODY()

	/** Draws a simple skeleton made up of lines rather than the full skeleton */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	bool bDrawSimpleSkeleton = false;

	/** Radius of the bones */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float BoneRadius = 1.0f;

	/** Number of segments in the spheres drawn for the bones */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (EditCondition = "!bDrawSimpleSkeleton", HideEditConditionToggle, ClampMin = "0", UIMin = "0"))
	int32 BoneSegmentNum = 10;

	/** If to draw small transforms inside the bone spheres */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (EditCondition = "!bDrawSimpleSkeleton", HideEditConditionToggle))
	bool bDrawTransforms = false;

	/** If to draw a bone going to the root/component transform */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	bool bDrawRoot = true;

	/** Color of the root bone when drawn */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (EditCondition = "bDrawRoot", HideEditConditionToggle))
	FLinearColor RootBoneColor = FLinearColor::Red;
};

/** Settings for debug drawing strings */
USTRUCT(BlueprintType, Category = "Draw Debug Library")
struct FDrawDebugStringSettings
{
	GENERATED_BODY()

	/** Height of the text */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float Height = 10.0f;

	/** If to draw using a monospaced font */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	bool bMonospaced = true;

	/** Scaling factor for the width of the characters */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0"))
	float WidthScale = 1.0f;

	/** Scaling factor for the height of the characters */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0"))
	float HeightScale = 1.0f;

	/** Scaling factor for the spacing between lines */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0"))
	float LineSpacing = 1.0f;

	/** Gaps between characters in cm */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0", Units = "cm"))
	float CharacterSpacing = 0.0f;

	/** Draws the string centered vertically rather than starting from the top left */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	bool bCenterVertically = false;

	/** Draws the string centered horizontally rather than starting from the top left */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	bool bCenterHorizontally = false;
};

/** Settings for debug drawing graphs axes */
USTRUCT(BlueprintType, Category = "Draw Debug Library")
struct FDrawDebugGraphAxesSettings
{
	GENERATED_BODY()

	/** Title of the graph */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	FString Title;

	/** Settings for the title */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	FDrawDebugStringSettings TitleSettings;

	/** Label for the X axis */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	FString XaxisLabel;

	/** Label for the Y axis */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	FString YaxisLabel;

	/** Settings for the axis arrows */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	FDrawDebugArrowSettings AxisArrowSettings;

	/** Settings for the axis labels */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	FDrawDebugStringSettings AxisLabelSettings;

	/** Draws the axes as a box instead of two arrows */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	bool bDrawAxesBox = false;
};

/** Settings for debug drawing canvas text */
USTRUCT(BlueprintType, Category = "Draw Debug Library")
struct FDrawDebugCanvasTextSettings
{
	GENERATED_BODY()

	/** If to draw using a monospaced font */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (EditCondition = "!Font", HideEditConditionToggle))
	bool bMonospaced = true;

	/** Custom font to use */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	TObjectPtr<UFont> Font = nullptr;

	/** Scaling factor for the width of the characters */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0"))
	float WidthScale = 1.0f;

	/** Scaling factor for the height of the characters */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (ClampMin = "0", UIMin = "0"))
	float HeightScale = 1.0f;

	/** If to draw a shadow under the text */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	bool bShadow = false;

	/** Color of the shadow */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (EditCondition = "bShadow", HideEditConditionToggle))
	FLinearColor ShadowColor = FLinearColor::Black;

	/** Offset of the shadow */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (EditCondition = "bShadow", HideEditConditionToggle))
	FVector2D ShadowOffset = FVector2D(1.0f, 1.0f);

	/** If to draw an outline around the text */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	bool bOutline = false;

	/** Color of the outline */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library", meta = (EditCondition = "bOutline", HideEditConditionToggle))
	FLinearColor OutlineColor = FLinearColor::White;

	/** Draws the text centered vertically rather than starting from the top left */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	bool bCenterVertically = false;

	/** Draws the text centered horizontally rather than starting from the top left */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Draw Debug Library")
	bool bCenterHorizontally = false;
};

/** This buffers debug drawing calls, which can later be dispatched to draw on a canvas using either the bound delegate or via manual flushing */
struct FDebugDrawerCanvasBuffer
{
	/** Destructor will un-bind the delegate if it is still bound */
	UE_API ~FDebugDrawerCanvasBuffer();

	/** Flushes the buffer and draws to the canvas */
	UE_API void Flush(FCanvas* Canvas);

	/** Delegate used to draw to a canvas somewhere */
	FDelegateHandle Delegate;

	/** Scale to draw to the screen */
	float Scale = 1.0f;

	/** Buffered primitive types. 0 - points, 1 - lines, 2 - text */
	TArray<uint8> PrimitiveTypes;

	/** Number of primitives to draw for each buffered call */
	TArray<int32> PrimitiveNums;

	/** Color of each buffered call */
	TArray<FLinearColor> Colors;

	/** Thickness of each buffered call */
	TArray<float> Thicknesses;

	/** Array of point data */
	TArray<FVector> Points;

	/** Array of line segment data */
	TArray<TPair<FVector, FVector>> LineSegments;

	/** Array of text data including draw location */
	TArray<TPair<FVector2D, FText>> Texts;

	/** Array of text setting data */
	TArray<FDrawDebugCanvasTextSettings> TextSettings;
};

/**
 * A simple Blueprint wrapper around different objects that are capable of debug drawing. This allows us to use the same blueprint interface and
 * functions for debug drawing if we are drawing using a UWorld object or a FPrimitiveDrawInterface object (for example). Since this contains raw 
 * pointers it should not be stored anywhere as these could become stale. It should always be treated as a temporary object.
 */
USTRUCT(BlueprintType, Category = "Draw Debug Library", meta = (HasNativeMake = "/Script/DrawDebugLibrary.DrawDebugLibrary.MakeDebugDrawer"))
struct FDebugDrawer
{
	GENERATED_BODY()

public: // Constructors

	/** Make a null debug drawer which ignores draw commands */
	UE_API FDebugDrawer();

	/** Make a debug drawer for a UObject */
	UE_API FDebugDrawer(UObject* InObject);

	/** Make a debug drawer for a UWorld */
	UE_API FDebugDrawer(UWorld* InWorld);

	/** Make a debug drawer for an AnimInstance */
	UE_API FDebugDrawer(UAnimInstance* InAnimInstance);

	/** Make a debug drawer for an AnimInstanceProxy */
	UE_API FDebugDrawer(FAnimInstanceProxy* InAnimInstanceProxy);

	/** Make a debug drawer for a FPrimitiveDrawInterface */
	UE_API FDebugDrawer(FPrimitiveDrawInterface* InPrimitiveDrawInterface);

	/** Make a debug drawer for a FRigVMDrawInterface */
	UE_API FDebugDrawer(FRigVMDrawInterface* InRigVMDrawInterface, const USceneComponent* InRigVMComponent = nullptr);

	/** Make a debug drawer for a FCanvas */
	UE_API FDebugDrawer(FCanvas* InCanvas);

	/** Make a debug drawer for a UCanvas */
	UE_API FDebugDrawer(UCanvas* InCanvas);

	/** Make a debug drawer for the given canvas buffer. */
	UE_API FDebugDrawer(const TSharedPtr<FDebugDrawerCanvasBuffer>& InCanvasBuffer);

	/** Forward declaration of destructor */
	UE_API ~FDebugDrawer();

	/** Make a null debug drawer which ignores draw commands */
	static UE_API FDebugDrawer MakeDebugDrawer();

	/** Make a debug drawer for a UObject */
	static UE_API FDebugDrawer MakeDebugDrawer(UObject* Object);

	/** Make a debug drawer for a UWorld */
	static UE_API FDebugDrawer MakeDebugDrawer(UWorld* World);

	/** Make a debug drawer for an AnimInstance */
	static UE_API FDebugDrawer MakeDebugDrawer(UAnimInstance* AnimInstance);

	/** Make a debug drawer for an AnimInstanceProxy */
	static UE_API FDebugDrawer MakeDebugDrawer(FAnimInstanceProxy* AnimInstanceProxy);

	/** Make a debug drawer for a FPrimitiveDrawInterface */
	static UE_API FDebugDrawer MakeDebugDrawer(FPrimitiveDrawInterface* PrimitiveDrawInterface);

	/** Make a debug drawer for a FRigVMDrawInterface */
	static UE_API FDebugDrawer MakeDebugDrawer(FRigVMDrawInterface* InRigVMDrawInterface, const USceneComponent* InRigVMComponent = nullptr);

	/** Make a debug drawer for a FCanvas */
	static UE_API FDebugDrawer MakeDebugDrawer(FCanvas* InCanvas, const float InCanvasScale = 1.0f);

	/** Make a debug drawer for a UCanvas */
	static UE_API FDebugDrawer MakeDebugDrawer(UCanvas* InCanvas, const float InCanvasScale = 1.0f);

	/** Make a debug drawer for the given canvas buffer. */
	static UE_API FDebugDrawer MakeDebugDrawer(const TSharedPtr<FDebugDrawerCanvasBuffer>& CanvasBuffer);

	/** Make a debug drawer for the Visual Logger */
	static UE_API FDebugDrawer MakeVisualLoggerDebugDrawer(
		const UObject* Object,
		const FName Category = TEXT("LogDrawDebugLibrary"), 
		const EDrawDebugLogVerbosity Verbosity = EDrawDebugLogVerbosity::Display, 
		const bool bDrawToScene = true,
		const bool bDrawToSceneWhileRecording = true);

	/** Make a debug drawer for the Visual Logger */
	static UE_API FDebugDrawer MakeVisualLoggerDebugDrawer(
		const UObject* Object, 
		const FLogCategoryBase& Category, 
		const ELogVerbosity::Type Verbosity = ELogVerbosity::Display, 
		const bool bDrawToScene = true,
		const bool bDrawToSceneWhileRecording = true);

	/** Make a debug drawer for the default viewport canvas. This can be expensive so should be called once and stored. */
	static UE_API FDebugDrawer MakeScreenDebugDrawer(const float Scale = 1.0f);

	/** Make a debug drawer by merging multiple other debug drawers of different types */
	static UE_API FDebugDrawer MakeMergedDebugDrawer(const TArrayView<const FDebugDrawer> DebugDrawers);

public: // Drawer specific interface

	UE_API void DrawPoints(const TArrayView<const FVector> Locations, const FLinearColor& Color, const float Thickness, const bool bDepthTest) const;
	UE_API void DrawLines(const TArrayView<const TPair<FVector, FVector>> Segments, const FLinearColor& Color, const float Thickness, const bool bDepthTest) const;

	UE_API void VisualLoggerLogString(const FStringView String, const FLinearColor& Color) const;
	UE_API void VisualLoggerDrawString(const FStringView String, const FVector& Location, const FLinearColor& Color) const;

	UE_API void CanvasDrawText(const FVector2D& Location, const FText& Text, const FLinearColor& Color, const FDrawDebugCanvasTextSettings& Settings) const;

private:

	/** Primitive Draw Interface - used for drawing in editor viewports. */
	FPrimitiveDrawInterface* PDI = nullptr;

private:

	/** RigVM Draw Interface - used for drawing in a RigVM context. */
	FRigVMDrawInterface* RigVM = nullptr;

	/** Component which the RigVM context is attached to */
	UPROPERTY()
	TObjectPtr<const USceneComponent> RigVMComponent = nullptr;

private:

	/** World object - used for normal debug drawing. */
	UPROPERTY()
	TObjectPtr<UWorld> World = nullptr;

private:

	/** Anim Instance Object - used for debug drawing in an AnimBlueprints. */
	UPROPERTY()
	TObjectPtr<UAnimInstance> AnimInstance = nullptr;

	/** Anim Instance Proxy Object - used for debug drawing in an AnimNode. */
	FAnimInstanceProxy* AnimInstanceProxy = nullptr;

private:

	/** Visual Logger Object - used for writing to the visual logger. */
	UPROPERTY()
	TObjectPtr<const UObject> VisualLoggerObject = nullptr;

	/** Visual Logger Category. */
	FName VisualLoggerCategory = NAME_None;

	/** Visual Logger Verbosity. */
	ELogVerbosity::Type VisualLoggerVerbosity = ELogVerbosity::Display;

	/** Visual Logger if to also draw to scene */
	bool bVisualLoggerDrawToScene = true;

	/** Visual Logger if to also draw while recording */
	bool bVisualLoggerDrawToSceneWhileRecording = true;

private:

	/** Canvas pointer - used for writing to a 2D canvas */
	FCanvas* Canvas = nullptr;

	/** Canvas Scale */
	float CanvasScale = 1.0f;

	/** Canvas buffer used for writing to a canvas at a later point */
	TSharedPtr<FDebugDrawerCanvasBuffer> CanvasBuffer;
};

/**
 * A blueprint library of additional debug draw functions that unifies drawing between different debug drawing interfaces. Also includes a number of 
 * helper functions and other convenience functions that are useful for debug drawing such as functions for recording histories.
 * 
 * General parameter structure for these functions is:
 * 
 *	Required Inputs, Style, bDepthTest, Settings
 */
UCLASS(BlueprintType, Category = "Draw Debug Library", meta = (BlueprintThreadSafe))
class UDrawDebugLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public: // Drawers

	/** Make a null debug drawer which ignores draw commands */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library")
	static UE_API FDebugDrawer MakeNullDebugDrawer();

	/** Make a debug debugger from the current "Self" object */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (DevelopmentOnly, DefaultToSelf = "Object", HidePin = "Object"))
	static UE_API FDebugDrawer MakeDebugDrawer(UObject* Object);

	/** Make a debug drawer for a UObject */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (DevelopmentOnly))
	static UE_API FDebugDrawer MakeObjectDebugDrawer(UObject* Object);

	/** Make a debug drawer for a UWorld */
	static UE_API FDebugDrawer MakeWorldDebugDrawer(UWorld* World);

	/** Make a debug drawer for an AnimInstance */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (DevelopmentOnly, DefaultToSelf = "AnimInstance"))
	static UE_API FDebugDrawer MakeAnimInstanceDebugDrawer(UAnimInstance* AnimInstance);

	/** Make a debug drawer for an AnimInstanceProxy */
	static UE_API FDebugDrawer MakeAnimInstanceProxyDebugDrawer(FAnimInstanceProxy* AnimInstanceProxy);

	/** Make a debug drawer for a FPrimitiveDrawInterface */
	static UE_API FDebugDrawer MakePDIDebugDrawer(FPrimitiveDrawInterface* PrimitiveDrawInterface);

	/** Make a debug drawer for a FRigVMDrawInterface */
	static UE_API FDebugDrawer MakeRigVMDebugDrawer(FRigVMDrawInterface* InRigVMDrawInterface, const USceneComponent* InRigVMComponent = nullptr);

	/** Make a debug drawer for a FCanvas */
	static UE_API FDebugDrawer MakeCanvasStructDebugDrawer(FCanvas* InCanvas, const float InScale = 1.0f);

	/** Make a debug drawer for a UCanvas */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (DevelopmentOnly))
	static UE_API FDebugDrawer MakeCanvasDebugDrawer(UCanvas* InCanvas, const float InScale = 1.0f);

	/** Make a debug drawer for the given buffer. */
	static UE_API FDebugDrawer MakeCanvasDebugDrawerFromBuffer(const TSharedPtr<FDebugDrawerCanvasBuffer>& CanvasBuffer);

	/** Make a debug drawer for the Visual Logger from the current "Self" object */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (DevelopmentOnly, DefaultToSelf = "Object", HidePin = "Object"))
	static UE_API FDebugDrawer MakeVisualLoggerDebugDrawer(
		const UObject* Object,
		const FName Category = TEXT("LogDrawDebugLibrary"), 
		const EDrawDebugLogVerbosity Verbosity = EDrawDebugLogVerbosity::Display, 
		const bool bDrawToScene = true,
		const bool bDrawToSceneWhileRecording = true);

	/** Make a debug drawer for the Visual Logger */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (DevelopmentOnly))
	static UE_API FDebugDrawer MakeVisualLoggerDebugDrawerFromObject(
		const UObject* Object,
		const FName Category = TEXT("LogDrawDebugLibrary"), 
		const EDrawDebugLogVerbosity Verbosity = EDrawDebugLogVerbosity::Display, 
		const bool bDrawToScene = true, 
		const bool bDrawToSceneWhileRecording = true);

	static UE_API FDebugDrawer MakeVisualLoggerDebugDrawerFromObjectWithCategory(
		const UObject* Object,
		const FLogCategoryBase& Category, 
		const ELogVerbosity::Type Verbosity = ELogVerbosity::Display, 
		const bool bDrawToScene = true, 
		const bool bDrawToSceneWhileRecording = true);

	/** Make a debug drawer for the default viewport canvas. This is expensive so should be called once and stored. */
	UFUNCTION(BlueprintPure = false, Category = "Draw Debug Library", meta = (DevelopmentOnly))
	static UE_API FDebugDrawer MakeScreenDebugDrawer(const float Scale = 1.0f);

	/** Make a debug drawer by merging multiple other debug drawers of different types */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (DevelopmentOnly))
	static UE_API FDebugDrawer MakeMergedDebugDrawer(const TArray<FDebugDrawer>& DebugDrawers);
	static UE_API FDebugDrawer MakeMergedDebugDrawerArrayView(const TArrayView<const FDebugDrawer> DebugDrawers);

public: // Visual Debugger Logging

	/** Log a string to the visual logger. Only does anything with a VisualLoggerDebugDrawer. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "String"))
	static UE_API void VisualLoggerLogString(const FDebugDrawer& Drawer, const FString& String, const FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f));
	static UE_API void VisualLoggerLogStringView(const FDebugDrawer& Drawer, const FStringView String, const FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f));

	/** Log some text to the visual logger. Only does anything with a VisualLoggerDebugDrawer. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Text"))
	static UE_API void VisualLoggerLogText(const FDebugDrawer& Drawer, const FText& Text, const FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f));
	
public: // Canvas Functions

	/** Draw some text on the canvas. Only does anything with a ScreenDebugDrawer or CanvasDebugDrawer. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Text, Settings"))
	static UE_API void DrawDebugCanvasText(const FDebugDrawer& Drawer, const FVector2D& Location, const FText& Text, const FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f), const FDrawDebugCanvasTextSettings& Settings = FDrawDebugCanvasTextSettings());

	/** Draw a string on the canvas. Only does anything with a ScreenDebugDrawer or CanvasDebugDrawer. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, String, Settings"))
	static UE_API void DrawDebugCanvasString(const FDebugDrawer& Drawer, const FVector2D& Location, const FString& String, const FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f), const FDrawDebugCanvasTextSettings& Settings = FDrawDebugCanvasTextSettings());
	static UE_API void DrawDebugCanvasStringView(const FDebugDrawer& Drawer, const FVector2D& Location, const FStringView String, const FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f), const FDrawDebugCanvasTextSettings& Settings = FDrawDebugCanvasTextSettings());

	/** Draw a name on the canvas. Only does anything with a ScreenDebugDrawer or CanvasDebugDrawer. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Name, Settings"))
	static UE_API void DrawDebugCanvasName(const FDebugDrawer& Drawer, const FVector2D& Location, const FName& Name, const FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f), const FDrawDebugCanvasTextSettings& Settings = FDrawDebugCanvasTextSettings());


public: // Convenience functions

	/** Convenience function for making an array of linearly spaced float values */
	UFUNCTION(BlueprintPure = true, Category = "Draw Debug Library")
	static UE_API void MakeLinearlySpacedFloatArray(TArray<float>& OutValues, const float Start = 0.0f, const float Stop = 1.0f, const int32 Num = 10);
	static UE_API void MakeLinearlySpacedFloatArrayView(TArrayView<float> OutValues, const float Start = 0.0f, const float Stop = 1.0f);

	/** Convenience function that adds a float to the end of an array, popping values from the front once the max is reached */
	UFUNCTION(BlueprintPure = false, Category = "Draw Debug Library", meta = (NotBlueprintThreadSafe))
	static UE_API void AddToFloatHistoryArray(UPARAM(ref) TArray<float>& InOutValues, const float NewValue = 0.0f, const int32 MaxHistoryNum = 60);

	/** Convenience function that adds a vector to the end of an array, popping values from the front once the max is reached */
	UFUNCTION(BlueprintPure = false, Category = "Draw Debug Library", meta = (NotBlueprintThreadSafe))
	static UE_API void AddToVectorHistoryArray(UPARAM(ref) TArray<FVector>& InOutValues, const FVector NewValue = FVector::ZeroVector, const int32 MaxHistoryNum = 60);

	/** Convenience function that adds a name to the end of an array, popping values from the front once the max is reached */
	UFUNCTION(BlueprintPure = false, Category = "Draw Debug Library", meta = (NotBlueprintThreadSafe))
	static UE_API void AddToNameHistoryArray(UPARAM(ref) TArray<FName>& InOutValues, const FName NewValue = NAME_None, const int32 MaxHistoryNum = 60);

	/** Returns a new location and rotation after applying a local offset to the input draw location and rotation */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library")
	static UE_API void DrawDebugLocalOffset(
		FVector& OutLocation,
		FRotator& OutRotation,
		const FVector DrawLocation,
		const FRotator DrawRotation,
		const FVector DrawOffset);

	/** Create a new location and rotation that orients debug drawing of upright objects such as text onto the floor */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library")
	static UE_API void DrawDebugOrientUprightToFloor(
		FVector& OutLocation,
		FRotator& OutRotation,
		const FVector DrawWorldLocation,
		const FVector DrawUprightOffset);

	/** Create a new location and rotation that orients debug drawing of floor objects such as shapes upright */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library")
	static UE_API void DrawDebugOrientFloorToUpright(
		FVector& OutLocation,
		FRotator& OutRotation,
		const FVector DrawWorldLocation,
		const FVector DrawFloorOffset);

	/** Create a new location and rotation that orients debug drawing of upright objects such as text to face the camera */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library")
	static UE_API void DrawDebugOrientUprightToCamera(
		FVector& OutLocation,
		FRotator& OutRotation,
		const FVector DrawWorldLocation,
		const FVector DrawUprightOffset,
		const FRotator CameraRotation,
		const bool bUseOnlyYaw = false);

	/** Create a new location and rotation that orients debug drawing of floor objects such as shapes to face the camera */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library")
	static UE_API void DrawDebugOrientFloorToCamera(
		FVector& OutLocation,
		FRotator& OutRotation,
		const FVector DrawWorldLocation,
		const FVector DrawFloorOffset,
		const FRotator CameraRotation,
		const bool bUseOnlyYaw = false);

	/** Create a new location and rotation that orients debug drawing of upright objects such as text and graphs to a canvas */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library")
	static UE_API void DrawDebugOrientUprightToCanvas(
		FVector& OutLocation,
		FRotator& OutRotation,
		const FVector2D DrawCanvasLocation);

public: // Style Modifiers

	/** Make a point style from a color */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (BlueprintAutocast, CompactNodeTitle = "->", AutoCreateRefTerm = "Color"))
	static UE_API FDrawDebugPointStyle MakeDrawDebugPointStyleFromColor(const FLinearColor& Color);

	/** Make a point style from a color and thickness */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library")
	static UE_API FDrawDebugPointStyle MakeDrawDebugPointStyleFromColorAndThickness(const FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f), const float Thickness = 0.0f);

	/** Make a line style from a color */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (BlueprintAutocast, CompactNodeTitle = "->", AutoCreateRefTerm = "Color"))
	static UE_API FDrawDebugLineStyle MakeDrawDebugLineStyleFromColor(const FLinearColor& Color);

	/** Make a line style from a color and thickness */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library")
	static UE_API FDrawDebugLineStyle MakeDrawDebugLineStyleFromColorAndThickness(const FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f), const float Thickness = 0.0f);

	/** Get a point style from a line style */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (BlueprintAutocast, CompactNodeTitle = "->", AutoCreateRefTerm = "LineStyle"))
	static UE_API FDrawDebugPointStyle DrawDebugPointStyleFromLineStyle(const FDrawDebugLineStyle& LineStyle);

	/** Get a line style from a point style */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (BlueprintAutocast, CompactNodeTitle = "->", AutoCreateRefTerm = "PointStyle"))
	static UE_API FDrawDebugLineStyle DrawDebugLineStyleFromPointStyle(const FDrawDebugPointStyle& PointStyle);

	/** Get the color from a point style */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (BlueprintAutocast, CompactNodeTitle = "->", AutoCreateRefTerm = "PointStyle"))
	static UE_API FLinearColor DrawDebugPointStyleColor(const FDrawDebugPointStyle& PointStyle);

	/** Get the color from a line style */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (BlueprintAutocast, CompactNodeTitle = "->", AutoCreateRefTerm = "LineStyle"))
	static UE_API FLinearColor DrawDebugLineStyleColor(const FDrawDebugLineStyle& LineStyle);

	/** Get a line style with the color changed */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (AutoCreateRefTerm = "LineStyle"))
	static UE_API FDrawDebugLineStyle DrawDebugLineStyleWithColor(const FDrawDebugLineStyle& LineStyle, const FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f));

	/** Get a line style with the color (excluding the opacity) changed */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (AutoCreateRefTerm = "LineStyle"))
	static UE_API FDrawDebugLineStyle DrawDebugLineStyleWithColorNoOpacity(const FDrawDebugLineStyle& LineStyle, const FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f));

	/** Get a line style with the thickness changed */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (AutoCreateRefTerm = "LineStyle"))
	static UE_API FDrawDebugLineStyle DrawDebugLineStyleWithThickness(const FDrawDebugLineStyle& LineStyle, const float Thickness = 0.0f);

	/** Get a line style with the type changed */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (AutoCreateRefTerm = "LineStyle"))
	static UE_API FDrawDebugLineStyle DrawDebugLineStyleWithType(const FDrawDebugLineStyle& LineStyle, const EDrawDebugLineType LineType = EDrawDebugLineType::Solid);

public: // Points

	/** Debug Draw a point */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, PointStyle"))
	static UE_API void DrawDebugPoint(const FDebugDrawer& Drawer, const FVector& Location, const FDrawDebugPointStyle& PointStyle = FDrawDebugPointStyle(), const bool bDepthTest = true);

	/** Debug Draw points. These should be preferred over DrawDebugPoint where possible as it will batch drawing when required such as when using the visual logger. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Locations, PointStyle"))
	static UE_API void DrawDebugPoints(const FDebugDrawer& Drawer, const TArray<FVector>& Locations, const FDrawDebugPointStyle& PointStyle = FDrawDebugPointStyle(), const bool bDepthTest = true);
	static UE_API void DrawDebugPointsArrayView(const FDebugDrawer& Drawer, const TArrayView<const FVector> Locations, const FDrawDebugPointStyle& PointStyle = FDrawDebugPointStyle(), const bool bDepthTest = true);

public: // Lines

	/** Debug Draw a line */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "StartLocation, EndLocation, LineStyle"))
	static UE_API void DrawDebugLine(const FDebugDrawer& Drawer, const FVector& StartLocation, const FVector& EndLocation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true);

	/** Debug Draw lines. These should be preferred over DrawDebugLine where possible as it will batch drawing when required such as when using the visual logger. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "StartLocations, EndLocations, LineStyle"))
	static UE_API void DrawDebugLines(const FDebugDrawer& Drawer, const TArray<FVector>& StartLocations, const TArray<FVector>& EndLocations, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true);
	static UE_API void DrawDebugLinesArrayView(const FDebugDrawer& Drawer, const TArrayView<const FVector> StartLocations, const TArrayView<const FVector> EndLocations, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true);
	static UE_API void DrawDebugLinesPairsArrayView(const FDebugDrawer& Drawer, const TArrayView<const TPair<FVector, FVector>> Segments, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true);

public: // Basic Shapes

	/** Debug Draw a triangular base pyramid */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugTriangularBasePyramid(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Length = 20.0f, const float Width = 20.0f);

	/** Debug Draw a square base pyramid */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugSquareBasePyramid(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Length = 20.0f, const float Width = 20.0f);

	/** Debug Draw a cone */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugCone(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Length = 20.0f, const float Radius = 10.0f, const int32 Segments = 9);

	/** Debug Draw a cone starting from the point oriented on the forward axis, with the given angle */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Direction, LineStyle"))
	static UE_API void DrawDebugConeLookAt(const FDebugDrawer& Drawer, const FVector& Location, const FVector& Direction = FVector::ForwardVector, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Length = 20.0f, const float Angle = 10.0f, const int32 Segments = 9);

	/** Debug Draw an arc around the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugArc(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const float Angle = 360.0f, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f, const int32 Segments = 17);

	/** Debug Draw an circle around the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugCircle(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f, const int32 Segments = 17);

	/** Debug Draw an tick on a circle at a given angle */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugCircleTick(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const float Angle, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f, const float Length = 2.5f, bool bInside = true);

	/** Debug Draw ticks on a circle at the given angles */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, Angles, LineStyle"))
	static UE_API void DrawDebugCircleTicks(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const TArray<float>& Angles, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f, const float Length = 5.0f, bool bInside = true);
	static UE_API void DrawDebugCircleTicksArrayView(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const TArrayView<const float> Angles, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f, const float Length = 5.0f, bool bInside = true);

	/** Debug Draw the outline of a circle (two circles) around the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugCircleOutline(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float InnerRadius = 10.0f, const float OuterRadius = 15.0f, const int32 Segments = 17);

	/** Debug Draw a triangle on the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugTriangle(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f);

	/** Debug Draw a square on the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugSquare(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float HalfLength = 10.0f);

	/** Debug Draw a cross on the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugCross(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float HalfLength = 10.0f);

	/** Debug Draw a diamond on the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugDiamond(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f);

	/** Debug Draw a pentagon on the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugPentagon(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f);

	/** Debug Draw a hexagon on the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugHexagon(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f);

	/** Debug Draw a heptagon on the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugHeptagon(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f);

	/** Debug Draw a octagon on the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugOctagon(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f);

	/** Debug Draw a nonagon on the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugNonagon(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f);

	/** Debug Draw a decagon on the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugDecagon(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f);

	/** Debug Draw a flat regular polygon on the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugRegularPolygon(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f, const int32 Sides = 11);

	/** Debug Draw a check mark on the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugCheckMark(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f);

	/** Debug Draw a checkbox on the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugCheckBox(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float HalfLength = 10.0f, const bool bChecked = false, const float TickRatio = 1.75f);

	/** Debug Draw a cross box on the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugCrossBox(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float HalfLength = 10.0f, const bool bCrossed = false, const float CrossRatio = 1.0);

	/** Debug Draw a radio button on the XY axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugRadioButton(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f, const bool bSelected = false, const float InnerButtonRatio = 0.5f, const int32 Segments = 17);

	/** Debug Draw an axis-aligned cross */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugLocator(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f);

	/** Debug Draw an axis-aligned cross rotated by 45 degrees */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugCrossLocator(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f);

	/** Debug Draw an oriented box */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugBox(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const FVector HalfExtents = FVector(10.0f, 10.0f, 10.0f));

	/** Debug Draw an FBox */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, Box"))
	static UE_API void DrawDebugBoxStruct(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const FBox& Box = FBox());

	/** Debug Draw a FBoxSphereBounds */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, Bounds"))
	static UE_API void DrawDebugBoxSphereBounds(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const FBoxSphereBounds& Bounds = FBoxSphereBounds());

	/** Debug Draw a sphere */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugSphere(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f, const int32 Segments = 8);

	/** Debug Draw a simple sphere made up of three circles one of each axis */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugSimpleSphere(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f, const int32 Segments = 13);

	/** Debug Draw a capsule */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugCapsule(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f, const float HalfLength = 10.0f, const int32 Segments = 8);

	/** Debug Draw a capsule using the start and end location */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "StartLocation, EndLocation, LineStyle"))
	static UE_API void DrawDebugCapsuleLine(const FDebugDrawer& Drawer, const FVector& StartLocation, const FVector& EndLocation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f, const int32 Segments = 8);

	/** Debug Draw a Frustum */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "FrustumToWorld, LineStyle"))
	static UE_API void DrawDebugFrustum(const FDebugDrawer& Drawer, const FMatrix& FrustumToWorld, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true);

public: // Arrows

	/** Debug Draw an arrow head */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugArrowHead(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const EDrawDebugArrowHead Type = EDrawDebugArrowHead::Simple, const float Size = 5.0f);

	/** Debug Draw an arrow */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "StartLocation, EndLocation, LineStyle, Settings"))
	static UE_API void DrawDebugArrow(const FDebugDrawer& Drawer, const FVector& StartLocation, const FVector& EndLocation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const FDrawDebugArrowSettings& Settings = FDrawDebugArrowSettings());

	/** Debug Draw an arrow with an orientation */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle, Settings"))
	static UE_API void DrawDebugOrientedArrow(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const float Length = 100.0f, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const FDrawDebugArrowSettings& Settings = FDrawDebugArrowSettings());

	/** Draws an arrow pointing down at the location surrounded by a circle */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, LineStyle"))
	static UE_API void DrawDebugGroundTargetArrow(const FDebugDrawer& Drawer, const FVector& Location, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Length = 100.0f, const float ArrowHeadSize = 20.0f, const float Radius = 10.0f, const int32 Segments = 17);

	/** Draws a flat 2d arrow motif */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugFlatArrow(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Length = 20.0f, const float Width = 20.0f);

	/** Debug Draw an arrow coming off a circle at a given angle */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle, Settings"))
	static UE_API void DrawDebugCircleArrow(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const float Angle, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f, const float Length = 20.0f, const FDrawDebugArrowSettings& Settings = FDrawDebugArrowSettings());

	/** Debug Draw an arc around the XY axis with arrow heads */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle, Settings"))
	static UE_API void DrawDebugArcArrow(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const float Angle = 90.0f, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 10.0f, const int32 Segments = 17, const FDrawDebugArrowSettings& Settings = FDrawDebugArrowSettings());

public: // Splines

	/** Debug Draw the start section of a catmull rom spline */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "V0, V1, V2, LineStyle"))
	static UE_API void DrawDebugCatmullRomSplineStart(const FDebugDrawer& Drawer, const FVector& V0, const FVector& V1, const FVector& V2, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const bool bMonotonic = false, const int32 Segments = 15);

	/** Debug Draw the end section of a catmull rom spline */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "V0, V1, V2, LineStyle"))
	static UE_API void DrawDebugCatmullRomSplineEnd(const FDebugDrawer& Drawer, const FVector& V0, const FVector& V1, const FVector& V2, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const bool bMonotonic = false, const int32 Segments = 15);

	/** Debug Draw a full segment of a catmull rom spline */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "V0, V1, V2, V3, LineStyle"))
	static UE_API void DrawDebugCatmullRomSplineSection(const FDebugDrawer& Drawer, const FVector& V0, const FVector& V1, const FVector& V2, const FVector& V3, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const bool bMonotonic = false, const int32 Segments = 15);

	/** Debug Draw a catmull rom spline */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Points, LineStyle"))
	static UE_API void DrawDebugCatmullRomSpline(const FDebugDrawer& Drawer, const TArray<FVector>& Points, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const bool bMonotonic = false, const int32 Segments = 15);
	static UE_API void DrawDebugCatmullRomSplineArrayView(const FDebugDrawer& Drawer, const TArrayView<const FVector> Points, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const bool bMonotonic = false, const int32 Segments = 15);

public: // Variable Types

	/** Debug Draw an angle */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugAngle(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const float Angle = 0.0f, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float LineLength = 10.0f, const float AngleRadius = 5.0f, const int32 Segments = 17);

	/** Debug Draw the angle between the vectors P0 - P1 and P2 - P1 */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "P0, P1, P2, LineStyle"))
	static UE_API void DrawDebugAngleBetween(const FDebugDrawer& Drawer, const FVector& P0, const FVector& P1, const FVector& P2, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float AngleRadius = 5.0f, const int32 Segments = 17);

	/** Debug Draw a sphere at the given location */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, LineStyle"))
	static UE_API void DrawDebugLocation(const FDebugDrawer& Drawer, const FVector& Location, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float DrawRadius = 10.0f, const int32 Segments = 8);

	/** Debug Draw spheres at the given locations */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Locations, LineStyle"))
	static UE_API void DrawDebugLocations(const FDebugDrawer& Drawer, const TArray<FVector>& Locations, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float DrawRadius = 10.0f, const int32 Segments = 8);
	static UE_API void DrawDebugLocationsArrayView(const FDebugDrawer& Drawer, const TArrayView<const FVector> Locations, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float DrawRadius = 10.0f, const int32 Segments = 8);

	/** Draws a rotation */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugRotation(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float DrawRadius = 10.0f);

	/** Draws an array of transforms */
	static UE_API void DrawDebugRotationsQuatArrayView(
		const FDebugDrawer& Drawer,
		const TArrayView<const FVector> Locations,
		const TArrayView<const FQuat4f> Rotations,
		const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(),
		const bool bDepthTest = true,
		const float DrawRadius = 10.0f);

	/** Debug Draw an arrow at the given location facing in the given direction */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Direction, LineStyle"))
	static UE_API void DrawDebugDirection(const FDebugDrawer& Drawer, const FVector& Location, const FVector& Direction = FVector::ForwardVector, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float DrawArrowLength = 100.0f, const float ArrowHeadScale = 1.0f);

	/** Debug Draw a line at the given location scaled by the given velocity */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Velocity, LineStyle"))
	static UE_API void DrawDebugVelocity(const FDebugDrawer& Drawer, const FVector& Location, const FVector& Velocity, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float DrawVelocityLineScale = 1.0f);

	/** Debug draw velocities */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Locations, Velocities, LineStyle"))
	static UE_API void DrawDebugVelocities(const FDebugDrawer& Drawer, const TArray<FVector>& Locations, const TArray<FVector>& Velocities, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float DrawVelocityLineScale = 1.0f);
	static UE_API void DrawDebugVelocitiesArrayView(const FDebugDrawer& Drawer, const TArrayView<const FVector> Locations, const TArrayView<const FVector> Velocities, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float DrawVelocityLineScale = 1.0f);

	/** Debug Draw a set of axes at the given transform */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Transform, LineStyle"))
	static UE_API void DrawDebugTransform(const FDebugDrawer& Drawer, const FTransform& Transform, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float DrawRadius = 10.0f);

	/** Draws an array of transforms */
	static UE_API void DrawDebugTransformsArrayView(
		const FDebugDrawer& Drawer,
		const TArrayView<const FVector> Locations,
		const TArrayView<const FQuat4f> Rotations,
		const TArrayView<const FVector3f> Scales,
		const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(),
		const bool bDepthTest = true,
		const float DrawRadius = 10.0f);

	/** Debug draw a phase-representation of an event at a location */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, LineStyle"))
	static UE_API void DrawDebugEvent(const FDebugDrawer& Drawer, bool bTimeUntilEventKnown, const float TimeUntilEvent, const FVector& Location, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Size = 20.0f);

public: // Complex Objects

	/** Debug draw a basic chair shape */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle, Settings"))
	static UE_API void DrawDebugChair(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const FDrawDebugChairSettings& Settings = FDrawDebugChairSettings());

	/** Debug draw a basic door shape */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle, Settings"))
	static UE_API void DrawDebugDoor(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const FDrawDebugDoorSettings& Settings = FDrawDebugDoorSettings());

	/** Debug draw a camera shape */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugCamera(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Scale = 10.0f, const float FOVDegrees = 30.0f);

	/** Debug draw a backpack */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle, Settings"))
	static UE_API void DrawDebugBackpack(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const FDrawDebugBackpackSettings& Settings = FDrawDebugBackpackSettings());

	/** Debug draw a briefcase */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle, Settings"))
	static UE_API void DrawDebugBriefcase(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const FDrawDebugBriefcaseSettings& Settings = FDrawDebugBriefcaseSettings());

	/** Debug Draw a trajectory of locations and directions */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Locations, Directions, RelativeTransform, LineStyle"))
	static UE_API void DrawDebugTrajectory(const FDebugDrawer& Drawer, const TArray<FVector>& Locations, const TArray<FVector>& Directions, const FTransform& RelativeTransform = FTransform(), const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float DrawArrowLength = 100.0f, const float PointRadius = 5.0f, const float ArrowHeadScale = 1.0f, const int32 Segments = 9, const float VerticalOffset = 2.0f);
	static UE_API void DrawDebugTrajectoryFromArrayViews(const FDebugDrawer& Drawer, const TArrayView<const FVector> Locations, const TArrayView<const FVector> Directions, const FTransform& RelativeTransform = FTransform(), const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float DrawArrowLength = 100.0f, const float PointRadius = 5.0f, const float ArrowHeadScale = 1.0f, const int32 Segments = 9, const float VerticalOffset = 2.0f);

	/** Debug Draw a transform trajectory */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "TransformTrajectory, LineStyle"))
	static UE_API void DrawDebugTransformTrajectory(const FDebugDrawer& Drawer, UPARAM(ref) const FTransformTrajectory& TransformTrajectory, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float Radius = 5.0f, const float VerticalOffset = 2.0f);

	/** Draws a trajectory in the style of trajectory drawing in persona */
	static UE_API void DrawDebugRangeTrajectoryArrayView(
		const FDebugDrawer& Drawer,
		const TArrayView<const FVector> TrajectoryLocations,
		const TArrayView<const FQuat4f> TrajectoryRotations,
		const bool bDepthTest = true,
		const FVector& ForwardVector = FVector::ForwardVector,
		const bool bDrawOrientations = false,
		const bool bDrawStartingAtOrigin = false,
		const FVector& OriginOffset = FVector::ZeroVector);

	/** Draws a flat 2d motif representing mover position and rotation */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle"))
	static UE_API void DrawDebugMoverOrientation(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const FVector ForwardVector = FVector::ForwardVector, const float Scale = 30.0f);

public: // Poses / Skeletons

	/** Gets the default debug draw bone color */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library")
	static UE_API FLinearColor GetDefaultBoneColor();

	/** Gets the default debug draw bone radius */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library")
	static UE_API float GetDefaultBoneRadius();

	/** Debug Draw a bone */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation"))
	static UE_API void DrawDebugBone(const FDebugDrawer& Drawer, const FVector& Location, const FRotator& Rotation, const FLinearColor Color = FLinearColor(0.0f, 0.0f, 0.025f, 1.0f), const bool bDepthTest = true, const float Radius = 5.0f, const int32 Segments = 10, const bool bDrawTransform = false);

	/** Debug Draw a link between a child and parent bone */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "ChildLocation, ParentLocation, ParentRotation"))
	static UE_API void DrawDebugBoneLink(const FDebugDrawer& Drawer, const FVector& ChildLocation, const FVector& ParentLocation, const FRotator& ParentRotation, const FLinearColor Color = FLinearColor(0.0f, 0.0f, 0.025f, 1.0f), const bool bDepthTest = true, const float Radius = 5.0f);

	/** Draws a skeleton from a skinned mesh component */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Settings"))
	static UE_API void DrawDebugSkeletonFromSkinnedMeshComponent(
		const FDebugDrawer& Drawer,
		const USkinnedMeshComponent* SkinnedMeshComponent,
		const FLinearColor Color = FLinearColor(0.0f, 0.0f, 0.025f, 1.0f),
		const bool bDepthTest = false,
		const FDrawDebugSkeletonSettings& Settings = FDrawDebugSkeletonSettings());

	/** Draws a skeleton made up of the given bone transforms */
	static UE_API void DrawDebugSkeletonArrayView(
		const FDebugDrawer& Drawer,
		const FVector RootLocation,
		const FQuat4f RootRotation,
		const TArrayView<const FVector> BoneLocations,
		const TArrayView<const FQuat4f> BoneRotations,
		const TArrayView<const int32> BoneIndices,
		const TArrayView<const int32> BoneParents,
		const FLinearColor Color = FLinearColor(0.0f, 0.0f, 0.025f, 1.0f),
		const bool bDepthTest = false,
		const FDrawDebugSkeletonSettings& Settings = FDrawDebugSkeletonSettings());

	/** Draws lines representing bone velocities */
	static UE_API void DrawDebugBoneVelocitiesArrayView(
		const FDebugDrawer& Drawer,
		const TArrayView<const FVector> BoneLocations,
		const TArrayView<const FVector3f> BoneVelocities,
		const TArrayView<const int32> BoneIndices,
		const FLinearColor Color = FLinearColor(0.0f, 0.0f, 0.025f, 1.0f),
		const float Thickness = 1.0f,
		const bool bDepthTest = false,
		const float Scale = 100.0f);

	/** Debug Draw a pose made up of bone locations and velocities */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "BoneLocations, BoneLinearVelocities, RelativeTransform, LineStyle"))
	static UE_API void DrawDebugPose(const FDebugDrawer& Drawer, const TArray<FVector>& BoneLocations, const TArray<FVector>& BoneLinearVelocities, const FTransform& RelativeTransform = FTransform(), const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float DrawVelocityLineScale = 1.0f);
	static UE_API void DrawDebugPoseFromArrayViews(const FDebugDrawer& Drawer, const TArrayView<const FVector> BoneLocations, const TArrayView<const FVector> BoneLinearVelocities, const FTransform& RelativeTransform = FTransform(), const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const float DrawVelocityLineScale = 1.0f);

	/** Draws all of the active curve values from a skeletal mesh component */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LineStyle, Settings"))
	static UE_API void DrawDebugCurvesFromSkeletalMeshComponent(
		const FDebugDrawer& Drawer,
		const USkeletalMeshComponent* SkeletalMeshComponent,
		const FVector& Location, 
		const FRotator& Rotation, 
		const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), 
		const bool bDepthTest = true,
		const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());

	/** Draws all of the active curve values from a skeletal mesh component as CanvasText. Only does anything with Screen or Canvas drawers. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Settings"))
	static UE_API void DrawDebugCurvesToCanvasFromSkeletalMeshComponent(
		const FDebugDrawer& Drawer,
		const USkeletalMeshComponent* SkeletalMeshComponent,
		const FVector2D& Location,
		const FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f),
		const FDrawDebugCanvasTextSettings& Settings = FDrawDebugCanvasTextSettings());

public: // Objects

	/** Debug Draw the bounding box of a static mesh */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation"))
	static UE_API void DrawDebugStaticMeshBoundingBox(const FDebugDrawer& Drawer, const UStaticMesh* StaticMesh, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true);


public: // Text

	/** Get the number of line segments required to draw debug a string. */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "String, Settings"))
	static UE_API int32 DrawDebugStringSegmentNum(const FString& String, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());
	static UE_API int32 DrawDebugStringViewSegmentNum(const FStringView String, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());

	/** Get the number of line segments required to draw debug text. */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Text, Settings"))
	static UE_API int32 DrawDebugTextSegmentNum(const FText& Text, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());

	/** Get the number of line segments required to draw debug a name. */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Name, Settings"))
	static UE_API int32 DrawDebugNameSegmentNum(const FName& Name, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());

	/** Get the dimensions of a draw debug string. Useful for aligning or centering text. */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "String, Settings"))
	static UE_API FVector DrawDebugStringDimensions(const FString& String, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());
	static UE_API FVector DrawDebugStringViewDimensions(const FStringView String, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());

	/** Get the dimensions of a draw debug text. Useful for aligning or centering text. */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Text, Settings"))
	static UE_API FVector DrawDebugTextDimensions(const FText& Text, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());

	/** Get the dimensions of a draw debug name. Useful for aligning or centering text. */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Name, Settings"))
	static UE_API FVector DrawDebugNameDimensions(const FName& Name, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());

	/** Get the local offset required to center a draw debug string. */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "String, Settings"))
	static UE_API FVector DrawDebugStringCenteringOffset(const FString& String, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());
	static UE_API FVector DrawDebugStringViewCenteringOffset(const FStringView String, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());

	/** Get the local offset required to center a draw debug text. */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Text, Settings"))
	static UE_API FVector DrawDebugTextCenteringOffset(const FText& Text, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());

	/** Get the local offset required to center a draw debug name. */
	UFUNCTION(BlueprintPure, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Name, Settings"))
	static UE_API FVector DrawDebugNameCenteringOffset(const FName& Name, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());

	/** Debug Draw a string. Will only render ASCII characters. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "String, Location, Rotation, LineStyle, Settings"))
	static UE_API void DrawDebugString(const FDebugDrawer& Drawer, const FString& String, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());
	static UE_API void DrawDebugStringView(const FDebugDrawer& Drawer, const FStringView String, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());

	/** Debug Draw a name */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Name, Location, Rotation, LineStyle, Settings"))
	static UE_API void DrawDebugName(const FDebugDrawer& Drawer, const FName& Name, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());

	/** Debug Draw several names */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Names, Location, Rotation, LineStyle, Settings"))
	static UE_API void DrawDebugNames(const FDebugDrawer& Drawer, const TArray<FName>& Names, const FVector& Location, const FRotator& Rotation, const FString& Separator = TEXT(", "), const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());
	static UE_API void DrawDebugNamesArrayView(const FDebugDrawer& Drawer, const TArrayView<const FName> Names, const FVector& Location, const FRotator& Rotation, const FStringView Separator = TEXT(", "), const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());

	/** Debug Draw some text. Will only render ASCII characters. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Text, Location, Rotation, LineStyle, Settings"))
	static UE_API void DrawDebugText(const FDebugDrawer& Drawer, const FText& Text, const FVector& Location, const FRotator& Rotation, const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(), const bool bDepthTest = true, const FDrawDebugStringSettings& Settings = FDrawDebugStringSettings());

public: // Visual Logger Text

	/** Debug Draw a string to the visual logger. Will do nothing if not using a VisualLoggerDrawer. Will not display on-screen during recording. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "String, Location"))
	static UE_API void VisualLoggerDrawString(const FDebugDrawer& Drawer, const FString& String, const FVector& Location, const FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f));
	static UE_API void VisualLoggerDrawStringView(const FDebugDrawer& Drawer, const FStringView String, const FVector& Location, const FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f));

	/** Debug Draw a name to the visual logger. Will do nothing if not using a VisualLoggerDrawer. Will not display on-screen during recording. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Name, Location"))
	static UE_API void VisualLoggerDrawName(const FDebugDrawer& Drawer, const FName& Name, const FVector& Location, const FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f));

	/** Debug Draw some text to the visual logger. Will do nothing if not using a VisualLoggerDrawer. Will not display on-screen during recording. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Text, Location"))
	static UE_API void VisualLoggerDrawText(const FDebugDrawer& Drawer, const FText& Text, const FVector& Location, const FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f));

public: // Graphs

	/** Debug Draw a simple graph. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, TextLineStyle, AxesLineStyle, PlotLineStyle, AxesSettings"))
	static UE_API void DrawDebugGraph(
		const FDebugDrawer& Drawer,
		const FVector& Location,
		const FRotator& Rotation,
		const TArray<float>& Xvalues,
		const TArray<float>& Yvalues,
		const float Xmin = 0.0f,
		const float Xmax = 1.0f,
		const float Ymin = 0.0f,
		const float Ymax = 1.0f,
		const float XaxisLength = 100.0f,
		const float YaxisLength = 100.0f,
		const FDrawDebugLineStyle& TextLineStyle = FDrawDebugLineStyle(),
		const FDrawDebugLineStyle& AxesLineStyle = FDrawDebugLineStyle(),
		const FDrawDebugLineStyle& PlotLineStyle = FDrawDebugLineStyle(),
		const bool bDepthTest = true,
		const FDrawDebugGraphAxesSettings& AxesSettings = FDrawDebugGraphAxesSettings());

	static UE_API void DrawDebugGraphArrayView(
		const FDebugDrawer& Drawer,
		const FVector& Location,
		const FRotator& Rotation,
		const TArrayView<const float> Xvalues,
		const TArrayView<const float> Yvalues,
		const float Xmin = 0.0f,
		const float Xmax = 1.0f,
		const float Ymin = 0.0f,
		const float Ymax = 1.0f,
		const float XaxisLength = 100.0f,
		const float YaxisLength = 100.0f,
		const FDrawDebugLineStyle& TextLineStyle = FDrawDebugLineStyle(),
		const FDrawDebugLineStyle& AxesLineStyle = FDrawDebugLineStyle(),
		const FDrawDebugLineStyle& PlotLineStyle = FDrawDebugLineStyle(),
		const bool bDepthTest = true,
		const FDrawDebugGraphAxesSettings& AxesSettings = FDrawDebugGraphAxesSettings());

	/** Debug Draw a simple graph axes labels. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, TextLineStyle, AxesSettings"))
	static UE_API void DrawDebugGraphAxesLabels(
		const FDebugDrawer& Drawer,
		const FVector& Location,
		const FRotator& Rotation,
		const float XaxisLength = 100.0f,
		const float YaxisLength = 100.0f,
		const FDrawDebugLineStyle& TextLineStyle = FDrawDebugLineStyle(),
		const bool bDepthTest = true,
		const FDrawDebugGraphAxesSettings& AxesSettings = FDrawDebugGraphAxesSettings());

	/** Debug Draw a simple graph axes. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, TextLineStyle, AxesLineStyle, AxesSettings"))
	static UE_API void DrawDebugGraphAxes(
		const FDebugDrawer& Drawer,
		const FVector& Location,
		const FRotator& Rotation,
		const float XaxisLength = 100.0f,
		const float YaxisLength = 100.0f,
		const FDrawDebugLineStyle& TextLineStyle = FDrawDebugLineStyle(),
		const FDrawDebugLineStyle& AxesLineStyle = FDrawDebugLineStyle(),
		const bool bDepthTest = true,
		const FDrawDebugGraphAxesSettings& AxesSettings = FDrawDebugGraphAxesSettings());

	/** Debug Draw a line on a simple graph. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, Xvalues, Yvalues, LineStyle"))
	static UE_API void DrawDebugGraphLine(
		const FDebugDrawer& Drawer,
		const FVector& Location,
		const FRotator& Rotation,
		const TArray<float>& Xvalues,
		const TArray<float>& Yvalues,
		const float Xmin = 0.0f,
		const float Xmax = 1.0f,
		const float Ymin = 0.0f,
		const float Ymax = 1.0f,
		const float XaxisLength = 100.0f,
		const float YaxisLength = 100.0f,
		const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(),
		const bool bDepthTest = true);

	static UE_API void DrawDebugGraphLineArrayView(
		const FDebugDrawer& Drawer,
		const FVector& Location,
		const FRotator& Rotation,
		const TArrayView<const float> Xvalues,
		const TArrayView<const float> Yvalues,
		const float Xmin = 0.0f,
		const float Xmax = 1.0f,
		const float Ymin = 0.0f,
		const float Ymax = 1.0f,
		const float XaxisLength = 100.0f,
		const float YaxisLength = 100.0f,
		const FDrawDebugLineStyle& LineStyle = FDrawDebugLineStyle(),
		const bool bDepthTest = true);

	/** Debug Draw a simple graph legend. */
	UFUNCTION(BlueprintCallable, Category = "Draw Debug Library", meta = (DevelopmentOnly, AutoCreateRefTerm = "Location, Rotation, LegendColors, LegendLabels, TextLineStyle, IconLineStyle, LegendSettings"))
	static UE_API void DrawDebugGraphLegend(
		const FDebugDrawer& Drawer,
		const FVector& Location,
		const FRotator& Rotation,
		const TArray<FLinearColor>& LegendColors,
		const TArray<FString>& LegendLabels,
		const float XaxisLength = 100.0f,
		const float YaxisLength = 100.0f,
		const FDrawDebugLineStyle& TextLineStyle = FDrawDebugLineStyle(),
		const FDrawDebugLineStyle& IconLineStyle = FDrawDebugLineStyle(),
		const float IconSize = 2.5f,
		const bool bDepthTest = true,
		const FDrawDebugStringSettings& LegendSettings = FDrawDebugStringSettings());

	static UE_API void DrawDebugGraphLegendArrayView(
		const FDebugDrawer& Drawer,
		const FVector& Location,
		const FRotator& Rotation,
		const TArrayView<const FLinearColor> LegendColors,
		const TArrayView<const FString> LegendLabels,
		const float XaxisLength = 100.0f,
		const float YaxisLength = 100.0f,
		const FDrawDebugLineStyle& TextLineStyle = FDrawDebugLineStyle(),
		const FDrawDebugLineStyle& IconLineStyle = FDrawDebugLineStyle(),
		const float IconSize = 2.5f,
		const bool bDepthTest = true,
		const FDrawDebugStringSettings& LegendSettings = FDrawDebugStringSettings());

};

#undef UE_API

