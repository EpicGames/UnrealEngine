// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DrawDebugLibrary.h"
#include "RigVMCore/RigVMStruct.h"
#include "Animation/TrajectoryTypes.h"

#include "RigUnit_DrawDebugLibrary.generated.h"

#define UE_API DRAWDEBUGLIBRARY_API

/*
 * The base class for all draw debug library functions
 */
USTRUCT(meta = (Abstract, Category = "Draw Debug Library", NodeColor = "0.83077 0.846873 0.049707", DocumentationPolicy = "Strict"))
struct FRigVMFunction_DrawDebugLibraryBase : public FRigVMStruct
{
	GENERATED_BODY()
};

/*
 * Base class for all draw debug library rig units
 */
USTRUCT(meta=(Abstract, ExecuteContext = "FRigVMExecuteContext", Category="Draw Debug Library", NodeColor = "0.83077 0.846873 0.049707", DocumentationPolicy = "Strict"))
struct FRigUnit_DrawDebugLibraryBase : public FRigVMStruct
{
	GENERATED_BODY()

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FRigVMExecuteContext ExecuteContext;
};

// Makes a null debug drawer
USTRUCT(meta = (DisplayName = "Make Null Debug Drawer"))
struct FRigVMFunction_MakeNullDebugDrawer : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Output))
	FDebugDrawer Drawer;
};

// Makes a normal RigVM debug drawer
USTRUCT(meta = (DisplayName = "Make Debug Drawer"))
struct FRigVMFunction_MakeDebugDrawer : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Output))
	FDebugDrawer Drawer;
};

// Makes a debug drawer for the visual logger.
USTRUCT(meta = (DisplayName = "Make Visual Logger Debug Drawer"))
struct FRigVMFunction_MakeVisualLoggerDebugDrawer : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Output))
	FDebugDrawer Drawer;

	// Log Category
	UPROPERTY(meta = (Input))
	FName Category = TEXT("LogDrawDebugLibrary");

	// Log Verbosity
	UPROPERTY(meta = (Input))
	EDrawDebugLogVerbosity Verbosity = EDrawDebugLogVerbosity::Display;
};

// Makes a merged debug drawer
USTRUCT(meta = (DisplayName = "Make Merged Debug Drawer"))
struct FRigVMFunction_MakeMergedDebugDrawer : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Output))
	FDebugDrawer Drawer;

	// Input Drawers
	UPROPERTY(meta = (Input))
	TArray<FDebugDrawer> Drawers;
};

// Log a string to the visual logger. Only does anything with a VisualLoggerDebugDrawer
USTRUCT(meta = (DisplayName = "Visual Logger Log Strong"))
struct FRigUnit_VisualLoggerLogString : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// String to log
	UPROPERTY(meta = (Input))
	FString String = TEXT("");

	// Log Color
	UPROPERTY(meta = (Input))
	FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f);
};

// Convenience function for making an array of linearly spaced float values 
USTRUCT(meta = (DisplayName = "Make Linearly Spaced Float Array"))
struct FRigVMFunction_MakeLinearlySpacedFloatArray : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Start Value
	UPROPERTY(meta = (Input))
	float Start = 0.0f;

	// Stop Value
	UPROPERTY(meta = (Input))
	float Stop = 1.0f;

	// Value Num
	UPROPERTY(meta = (Input))
	float Num = 10;

	// Output Values
	UPROPERTY(meta = (Output))
	TArray<float> Values;
};

// Convenience function that adds a float to the end of an array, popping values from the front once the max is reached
USTRUCT(meta = (DisplayName = "Add to Float History Array"))
struct FRigUnit_AddToFloatHistoryArray : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// New value to add
	UPROPERTY(meta = (Input))
	float NewValue = 0.0f;

	// Maximum number of values
	UPROPERTY(meta = (Input))
	int32 MaxHistoryNum = 60;

	// Log Color
	UPROPERTY(meta = (Input, Output))
	TArray<float> Values;
};

// Convenience function that adds a vector to the end of an array, popping values from the front once the max is reached
USTRUCT(meta = (DisplayName = "Add to Vector History Array"))
struct FRigUnit_AddToVectorHistoryArray : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// New value to add
	UPROPERTY(meta = (Input))
	FVector NewValue = FVector::ZeroVector;

	// Maximum number of values
	UPROPERTY(meta = (Input))
	int32 MaxHistoryNum = 60;

	// Log Color
	UPROPERTY(meta = (Input, Output))
	TArray<FVector> Values;
};

// Convenience function that adds a name to the end of an array, popping values from the front once the max is reached
USTRUCT(meta = (DisplayName = "Add to Name History Array"))
struct FRigUnit_AddToNameHistoryArray : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// New value to add
	UPROPERTY(meta = (Input))
	FName NewValue = NAME_None;

	// Maximum number of values
	UPROPERTY(meta = (Input))
	int32 MaxHistoryNum = 60;

	// Log Color
	UPROPERTY(meta = (Input, Output))
	TArray<FName> Values;
};

// Returns a new location and rotation after applying a local offset to the input draw location and rotation 
USTRUCT(meta = (DisplayName = "Draw Debug Local Offset"))
struct FRigVMFunction_DrawDebugLocalOffset : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Input Draw Location
	UPROPERTY(meta = (Input))
	FVector DrawLocation = FVector::ZeroVector;

	// Input Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator DrawRotation = FRotator::ZeroRotator;

	// Offset in the local space
	UPROPERTY(meta = (Input))
	FVector DrawOffset = FVector::ZeroVector;

	// Output Location
	UPROPERTY(meta = (Output))
	FVector Location = FVector::ZeroVector;

	// Output Rotation
	UPROPERTY(meta = (Output))
	FRotator Rotation = FRotator::ZeroRotator;
};

// Returns a new location and rotation after applying a local offset to the input draw location and rotation 
USTRUCT(meta = (DisplayName = "Draw Debug Orient Upright to Floor"))
struct FRigVMFunction_DrawDebugOrientUprightToFloor : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Draw location
	UPROPERTY(meta = (Input))
	FVector DrawWorldLocation = FVector::ZeroVector;

	// Draw offset in local space
	UPROPERTY(meta = (Input))
	FVector DrawUprightOffset = FVector::ZeroVector;

	// Output Location
	UPROPERTY(meta = (Output))
	FVector Location = FVector::ZeroVector;

	// Output Rotation
	UPROPERTY(meta = (Output))
	FRotator Rotation = FRotator::ZeroRotator;
};

// Create a new location and rotation that orients debug drawing of floor objects such as shapes upright
USTRUCT(meta = (DisplayName = "Draw Debug Orient Floor to Upright"))
struct FRigVMFunction_DrawDebugOrientFloorToUpright : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Draw location
	UPROPERTY(meta = (Input))
	FVector DrawWorldLocation = FVector::ZeroVector;

	// Draw offset in local space
	UPROPERTY(meta = (Input))
	FVector DrawFloorOffset = FVector::ZeroVector;

	// Output Location
	UPROPERTY(meta = (Output))
	FVector Location = FVector::ZeroVector;

	// Output Rotation
	UPROPERTY(meta = (Output))
	FRotator Rotation = FRotator::ZeroRotator;
};

// Create a new location and rotation that orients debug drawing of upright objects such as text to face the camera
USTRUCT(meta = (DisplayName = "Draw Debug Orient Upright to Camera"))
struct FRigVMFunction_DrawDebugOrientUprightToCamera : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Draw location
	UPROPERTY(meta = (Input))
	FVector DrawWorldLocation = FVector::ZeroVector;

	// Draw offset in local space
	UPROPERTY(meta = (Input))
	FVector DrawUprightOffset = FVector::ZeroVector;

	// Camera Rotation
	UPROPERTY(meta = (Input))
	FRotator CameraRotation = FRotator::ZeroRotator;

	// If to only oriented to the camera yaw
	UPROPERTY(meta = (Input))
	bool bUseOnlyYaw = false;

	// Output Location
	UPROPERTY(meta = (Output))
	FVector Location = FVector::ZeroVector;

	// Output Rotation
	UPROPERTY(meta = (Output))
	FRotator Rotation = FRotator::ZeroRotator;
};

// Create a new location and rotation that orients debug drawing of floor objects such as shapes to face the camera
USTRUCT(meta = (DisplayName = "Draw Debug Orient Floor to Camera"))
struct FRigVMFunction_DrawDebugOrientFloorToCamera : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Draw location
	UPROPERTY(meta = (Input))
	FVector DrawWorldLocation = FVector::ZeroVector;

	// Draw offset in local space
	UPROPERTY(meta = (Input))
	FVector DrawFloorOffset = FVector::ZeroVector;

	// Camera Rotation
	UPROPERTY(meta = (Input))
	FRotator CameraRotation = FRotator::ZeroRotator;

	// If to only oriented to the camera yaw
	UPROPERTY(meta = (Input))
	bool bUseOnlyYaw = false;

	// Output Location
	UPROPERTY(meta = (Output))
	FVector Location = FVector::ZeroVector;

	// Output Rotation
	UPROPERTY(meta = (Output))
	FRotator Rotation = FRotator::ZeroRotator;
};

// Debug Draw a point
USTRUCT(meta = (DisplayName = "Draw Debug Point"))
struct FRigUnit_DrawDebugPoint : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Point Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Point Style
	UPROPERTY(meta = (Input))
	FDrawDebugPointStyle PointStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;
};

// Debug Draw points. These should be preferred over DrawDebugPoint where possible as it will batch drawing when required such as when using the visual logger. 
USTRUCT(meta = (DisplayName = "Draw Debug Points"))
struct FRigUnit_DrawDebugPoints : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Point Locations
	UPROPERTY(meta = (Input))
	TArray<FVector> Locations;

	// Point Style
	UPROPERTY(meta = (Input))
	FDrawDebugPointStyle PointStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;
};

// Debug Draws a line
USTRUCT(meta=(DisplayName="Draw Debug Line"))
struct FRigUnit_DrawDebugLine : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
    FDebugDrawer Drawer;

	// Line Start Location
	UPROPERTY(meta = (Input))
    FVector StartLocation = FVector::ZeroVector;

	// Line End Location
	UPROPERTY(meta = (Input))
    FVector EndLocation = FVector::ZeroVector;

	// Line Style
	UPROPERTY(meta = (Input))
    FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
    bool bDepthTest = true;
};

// Debug Draws lines. These should be preferred over DrawDebugLine where possible as it will batch drawing when required such as when using the visual logger. 
USTRUCT(meta = (DisplayName = "Draw Debug Lines"))
struct FRigUnit_DrawDebugLines : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Line Start Locations
	UPROPERTY(meta = (Input))
	TArray<FVector> StartLocations;

	// Line End Locations
	UPROPERTY(meta = (Input))
	TArray<FVector> EndLocations;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;
};

// Debug Draw a triangular base pyramid
USTRUCT(meta = (DisplayName = "Draw Debug Triangular Base Pyramid"))
struct FRigUnit_DrawDebugTriangularBasePyramid : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Pyramid Length
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Length = 20.0f;

	// Pyramid Width
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Width = 20.0f;
};

// Debug Draw a square base pyramid
USTRUCT(meta = (DisplayName = "Draw Debug Square Base Pyramid"))
struct FRigUnit_DrawDebugSquareBasePyramid : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Pyramid Length
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Length = 20.0f;

	// Pyramid Width
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Width = 20.0f;
};

// Debug Draw a cone
USTRUCT(meta = (DisplayName = "Draw Debug Cone"))
struct FRigUnit_DrawDebugCone : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Cone Length
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Length = 20.0f;

	// Cone Radius
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;

	// Number of segments
	UPROPERTY(meta = (Input))
	int32 Segments = 9;
};

// Debug Draw a cone starting from the point oriented on the forward axis, with the given angle
USTRUCT(meta = (DisplayName = "Draw Debug Cone Look At"))
struct FRigUnit_DrawDebugConeLookAt : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Direction
	UPROPERTY(meta = (Input))
	FVector Direction = FVector::ForwardVector;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Cone Length
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Length = 20.0f;

	// Cone Angle
	UPROPERTY(meta = (Input, ForceUnits = "deg"))
	float Angle = 10.0f;

	// Number of segments
	UPROPERTY(meta = (Input))
	int32 Segments = 9;
};

// Debug Draw an arc around the XY axis
USTRUCT(meta = (DisplayName = "Draw Debug Arc"))
struct FRigUnit_DrawDebugArc : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Arc Angle
	UPROPERTY(meta = (Input, ForceUnits="deg"))
	float Angle = 360.0f;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Arc Radius
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;

	// Number of segments
	UPROPERTY(meta = (Input))
	int32 Segments = 17;
};

// Debug Draw an circle around the XY axis
USTRUCT(meta = (DisplayName = "Draw Debug Circle"))
struct FRigUnit_DrawDebugCircle : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Circle Radius
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;

	// Number of segments
	UPROPERTY(meta = (Input))
	int32 Segments = 17;
};

// Debug Draw an tick on a circle at a given angle
USTRUCT(meta = (DisplayName = "Draw Debug Circle Tick"))
struct FRigUnit_DrawDebugCircleTick : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Tick Angle
	UPROPERTY(meta = (Input, ForceUnits = "deg"))
	float Angle = 0.0f;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Circle Radius
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;

	// Tick Length Radius
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Length = 2.5f;

	// If to draw the tick on the inside or outside
	UPROPERTY(meta = (Input))
	bool bInside = true;
};

// Debug Draw ticks on a circle at the given angles
USTRUCT(meta = (DisplayName = "Draw Debug Circle Ticks"))
struct FRigUnit_DrawDebugCircleTicks : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Tick Angles
	UPROPERTY(meta = (Input))
	TArray<float> Angles;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Circle Radius
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;

	// Tick Length Radius
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Length = 2.5f;

	// If to draw the tick on the inside or outside
	UPROPERTY(meta = (Input))
	bool bInside = true;
};


// Debug Draw the outline of a circle (two circles) around the XY axis
USTRUCT(meta = (DisplayName = "Draw Debug Circle Outline"))
struct FRigUnit_DrawDebugCircleOutline : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Inner Circle Radius
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float InnerRadius = 10.0f;

	// Outer Circle Radius
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float OuterRadius = 15.0f;

	// Number of segments
	UPROPERTY(meta = (Input))
	int32 Segments = 17;
};

// Debug Draw a triangle on the XY axis
USTRUCT(meta = (DisplayName = "Draw Debug Triangle"))
struct FRigUnit_DrawDebugTriangle : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Triangle Radius
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;
};

// Debug Draw a square on the XY axis
USTRUCT(meta = (DisplayName = "Draw Debug Square"))
struct FRigUnit_DrawDebugSquare : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Half-length of the square
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float HalfLength = 10.0f;
};

// Debug Draw a cross on the XY axis
USTRUCT(meta = (DisplayName = "Draw Debug Cross"))
struct FRigUnit_DrawDebugCross : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Half-length of the cross
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float HalfLength = 10.0f;
};

// Debug Draw a diamond on the XY axis
USTRUCT(meta = (DisplayName = "Draw Debug Diamond"))
struct FRigUnit_DrawDebugDiamond : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the shape
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;
};

// Debug Draw a pentagon on the XY axis
USTRUCT(meta = (DisplayName = "Draw Debug Pentagon"))
struct FRigUnit_DrawDebugPentagon : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the shape
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;
};

// Debug Draw a hexagon on the XY axis
USTRUCT(meta = (DisplayName = "Draw Debug Hexagon"))
struct FRigUnit_DrawDebugHexagon : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the shape
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;
};

// Debug Draw a heptagon on the XY axis
USTRUCT(meta = (DisplayName = "Draw Debug Heptagon"))
struct FRigUnit_DrawDebugHeptagon : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the shape
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;
};

// Debug Draw a octagon on the XY axis
USTRUCT(meta = (DisplayName = "Draw Debug Octagon"))
struct FRigUnit_DrawDebugOctagon : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the shape
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;
};

// Debug Draw a nonagon on the XY axis
USTRUCT(meta = (DisplayName = "Draw Debug Nonagon"))
struct FRigUnit_DrawDebugNonagon : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the shape
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;
};

// Debug Draw a decagon on the XY axis
USTRUCT(meta = (DisplayName = "Draw Debug Decagon"))
struct FRigUnit_DrawDebugDecagon : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the shape
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;
};

// Debug Draw a flat regular polygon on the XY axis
USTRUCT(meta = (DisplayName = "Draw Debug Regular Polygon"))
struct FRigUnit_DrawDebugRegularPolygon : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the shape
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;

	// Number of sides
	UPROPERTY(meta = (Input))
	int32 Sides = 11;
};

// Debug Draw a check mark on the XY axis
USTRUCT(meta = (DisplayName = "Draw Debug Check Mark"))
struct FRigUnit_DrawDebugCheckMark : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the shape
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;
};

// Debug Draw a checkbox on the XY axis 
USTRUCT(meta = (DisplayName = "Draw Debug Check Box"))
struct FRigUnit_DrawDebugCheckBox : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Half-length of the check box
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float HalfLength = 10.0f;

	// If the check box is checked
	UPROPERTY(meta = (Input))
	bool bChecked = false;

	// Size of the tick mark in proportion to the box
	UPROPERTY(meta = (Input))
	float TickRatio = 1.75f;
};

// Debug Draw a cross box on the XY axis
USTRUCT(meta = (DisplayName = "Draw Debug Cross Box"))
struct FRigUnit_DrawDebugCrossBox : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Half-length of the cross box
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float HalfLength = 10.0f;

	// If the cross box is marked
	UPROPERTY(meta = (Input))
	bool bCrossed = false;

	// Size of the cross mark in proportion to the box
	UPROPERTY(meta = (Input))
	float CrossRatio = 1.0f;
};

// Debug Draw a radio button on the XY axis
USTRUCT(meta = (DisplayName = "Draw Debug Radio Button"))
struct FRigUnit_DrawDebugRadioButton : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the radio button
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;

	// If the radio button is selected marked
	UPROPERTY(meta = (Input))
	bool bSelected = false;

	// Ratio of the inner button compared to the outer circle
	UPROPERTY(meta = (Input))
	float InnerButtonRatio = 0.5f;

	// Number of segments
	UPROPERTY(meta = (Input))
	int32 Segments = 17;
};

// Debug Draw an axis-aligned cross
USTRUCT(meta = (DisplayName = "Draw Debug Locator"))
struct FRigUnit_DrawDebugLocator : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the radio button
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;
};

// Debug Draw an axis-aligned cross rotated by 45 degrees
USTRUCT(meta = (DisplayName = "Draw Debug Cross Locator"))
struct FRigUnit_DrawDebugCrossLocator : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the radio button
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;
};

// Debug Draw an oriented box
USTRUCT(meta = (DisplayName = "Draw Debug Box"))
struct FRigUnit_DrawDebugBox : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Half-extents of the box
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	FVector HalfExtents = FVector(10.0f, 10.0f, 10.0f);
};

// Debug Draw a sphere
USTRUCT(meta = (DisplayName = "Draw Debug Sphere"))
struct FRigUnit_DrawDebugSphere : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the sphere
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;

	// Number of segments
	UPROPERTY(meta = (Input))
	int32 Segments = 8;
};

// Debug Draw a simple sphere made up of three circles one of each axis
USTRUCT(meta = (DisplayName = "Draw Debug Simple Sphere"))
struct FRigUnit_DrawDebugSimpleSphere : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the sphere
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;

	// Number of segments
	UPROPERTY(meta = (Input))
	int32 Segments = 13;
};

// Debug Draw a capsule
USTRUCT(meta = (DisplayName = "Draw Debug Capsule"))
struct FRigUnit_DrawDebugCapsule : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the sphere
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;

	// Half-Length of the capsule
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float HalfLength = 10.0f;

	// Number of segments
	UPROPERTY(meta = (Input))
	int32 Segments = 8;
};

// Debug Draw a capsule using the start and end location
USTRUCT(meta = (DisplayName = "Draw Debug Capsule Line"))
struct FRigUnit_DrawDebugCapsuleLine : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Start Location
	UPROPERTY(meta = (Input))
	FVector StartLocation = FVector::ZeroVector;

	// Draw End Location
	UPROPERTY(meta = (Input))
	FVector EndLocation = FVector::ZeroVector;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the sphere
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;

	// Number of segments
	UPROPERTY(meta = (Input))
	int32 Segments = 8;
};

// Debug Draw a Frustum
USTRUCT(meta = (DisplayName = "Draw Debug Frustum"))
struct FRigUnit_DrawDebugFrustum : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Frustum Matrix
	UPROPERTY(meta = (Input))
	FMatrix FrustumToWorld = FMatrix::Identity;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;
};

// Debug Draw an arrow head
USTRUCT(meta = (DisplayName = "Draw Debug Arrow Head"))
struct FRigUnit_DrawDebugArrowHead : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Arrow Head Type
	UPROPERTY(meta = (Input))
	EDrawDebugArrowHead Type = EDrawDebugArrowHead::Simple;
	
	// Arrow Head Size
	UPROPERTY(meta = (Input))
	float Size = 5.0f;
};

// Debug Draw an arrow
USTRUCT(meta = (DisplayName = "Draw Debug Arrow"))
struct FRigUnit_DrawDebugArrow : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Start Location
	UPROPERTY(meta = (Input))
	FVector StartLocation = FVector::ZeroVector;

	// Draw End Location
	UPROPERTY(meta = (Input))
	FVector EndLocation = FVector::ZeroVector;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Arrow Settings
	UPROPERTY(meta = (Input))
	FDrawDebugArrowSettings Settings;
};

// Debug Draw an arrow with an orientation
USTRUCT(meta = (DisplayName = "Draw Debug Oriented Arrow"))
struct FRigUnit_DrawDebugOrientedArrow : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Length of the arrow
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Length = 100.0f;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Arrow Settings
	UPROPERTY(meta = (Input))
	FDrawDebugArrowSettings Settings;
};

// Draws an arrow pointing down at the location surrounded by a circle
USTRUCT(meta = (DisplayName = "Draw Debug Ground Target Arrow"))
struct FRigUnit_DrawDebugGroundTargetArrow : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Length of the arrow
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Length = 100.0f;

	// Size of the arrow head
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float ArrowHeadSize = 20.0f;

	// Radius of the target circle
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;

	// Number of segments of the target circle
	UPROPERTY(meta = (Input))
	int32 Segments = 17;
};

// Draws a flat 2d arrow motif
USTRUCT(meta = (DisplayName = "Draw Debug Flat Arrow"))
struct FRigUnit_DrawDebugFlatArrow : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Length of the arrow
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Length = 20.0f;

	// Width of the arrow
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Width = 20.0f;
};

// Debug Draw an arrow coming off a circle at a given angle
USTRUCT(meta = (DisplayName = "Draw Debug Circle Arrow"))
struct FRigUnit_DrawDebugCircleArrow : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Angle at which to draw the arrow
	UPROPERTY(meta = (Input, ForceUnits = "deg"))
	float Angle = 0.0f;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the circle
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;

	// Length of the arrow
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Length = 20.0f;

	// Arrow Settings
	UPROPERTY(meta = (Input))
	FDrawDebugArrowSettings Settings;
};

// Debug Draw an arc around the XY axis with arrow heads
USTRUCT(meta = (DisplayName = "Draw Debug Arc Arrow"))
struct FRigUnit_DrawDebugArcArrow : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Angle at which to draw the arrow
	UPROPERTY(meta = (Input, ForceUnits = "deg"))
	float Angle = 0.0f;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the circle
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 10.0f;

	// Number of segments in the arc
	UPROPERTY(meta = (Input))
	int32 Segments = 17;

	// Arrow Settings
	UPROPERTY(meta = (Input))
	FDrawDebugArrowSettings Settings;
};

// Debug Draw the start section of a catmull rom spline
USTRUCT(meta = (DisplayName = "Draw Debug Catmull Rom Spline Start"))
struct FRigUnit_DrawDebugCatmullRomSplineStart : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Vertex 0
	UPROPERTY(meta = (Input))
	FVector V0 = FVector::ZeroVector;

	// Vertex 1
	UPROPERTY(meta = (Input))
	FVector V1 = FVector::ZeroVector;

	// Vertex 2
	UPROPERTY(meta = (Input))
	FVector V2 = FVector::ZeroVector;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// If to draw a monotonic spline
	UPROPERTY(meta = (Input))
	bool bMonotonic = false;

	// Number of segments in the spline segment
	UPROPERTY(meta = (Input))
	int32 Segments = 15;
};

// Debug Draw the end section of a catmull rom spline
USTRUCT(meta = (DisplayName = "Draw Debug Catmull Rom Spline End"))
struct FRigUnit_DrawDebugCatmullRomSplineEnd : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Vertex 0
	UPROPERTY(meta = (Input))
	FVector V0 = FVector::ZeroVector;

	// Vertex 1
	UPROPERTY(meta = (Input))
	FVector V1 = FVector::ZeroVector;

	// Vertex 2
	UPROPERTY(meta = (Input))
	FVector V2 = FVector::ZeroVector;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// If to draw a monotonic spline
	UPROPERTY(meta = (Input))
	bool bMonotonic = false;

	// Number of segments in the spline segment
	UPROPERTY(meta = (Input))
	int32 Segments = 15;
};

// Debug Draw a full segment of a catmull rom spline
USTRUCT(meta = (DisplayName = "Draw Debug Catmull Rom Spline Section"))
struct FRigUnit_DrawDebugCatmullRomSplineSection : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Vertex 0
	UPROPERTY(meta = (Input))
	FVector V0 = FVector::ZeroVector;

	// Vertex 1
	UPROPERTY(meta = (Input))
	FVector V1 = FVector::ZeroVector;

	// Vertex 2
	UPROPERTY(meta = (Input))
	FVector V2 = FVector::ZeroVector;

	// Vertex 3
	UPROPERTY(meta = (Input))
	FVector V3 = FVector::ZeroVector;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// If to draw a monotonic spline
	UPROPERTY(meta = (Input))
	bool bMonotonic = false;

	// Number of segments in the spline segment
	UPROPERTY(meta = (Input))
	int32 Segments = 15;
};

// Debug Draw a catmull rom spline
USTRUCT(meta = (DisplayName = "Draw Debug Catmull Rom Spline"))
struct FRigUnit_DrawDebugCatmullRomSpline : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Spline Points
	UPROPERTY(meta = (Input))
	TArray<FVector> Points;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// If to draw a monotonic spline
	UPROPERTY(meta = (Input))
	bool bMonotonic = false;

	// Number of segments in the spline segment
	UPROPERTY(meta = (Input))
	int32 Segments = 15;
};


// Debug Draw an angle
USTRUCT(meta = (DisplayName = "Draw Debug Angle"))
struct FRigUnit_DrawDebugAngle : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Angle to draw
	UPROPERTY(meta = (Input, ForceUnits = "deg"))
	float Angle = 0.0f;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Length of the line
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float LineLength = 10.0f;

	// Radius of the angular arc
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float AngleRadius = 10.0f;

	// Number of segments in the arc
	UPROPERTY(meta = (Input))
	int32 Segments = 17;
};

// Debug Draw the angle between the vectors P0 - P1 and P2 - P1
USTRUCT(meta = (DisplayName = "Draw Debug Angle Between"))
struct FRigUnit_DrawDebugAngleBetween : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Point P0
	UPROPERTY(meta = (Input))
	FVector P0 = FVector::ZeroVector;

	// Draw Point P1
	UPROPERTY(meta = (Input))
	FVector P1 = FVector::ZeroVector;

	// Draw Point P2
	UPROPERTY(meta = (Input))
	FVector P2 = FVector::ZeroVector;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the angular arc
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float AngleRadius = 10.0f;

	// Number of segments in the arc
	UPROPERTY(meta = (Input))
	int32 Segments = 17;
};

// Debug Draw a sphere at the given location
USTRUCT(meta = (DisplayName = "Draw Debug Location"))
struct FRigUnit_DrawDebugLocation : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the sphere to draw
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float DrawRadius = 10.0f;

	// Number of segments in the sphere
	UPROPERTY(meta = (Input))
	int32 Segments = 8;
};

// Debug Draw spheres at the given locations
USTRUCT(meta = (DisplayName = "Draw Debug Locations"))
struct FRigUnit_DrawDebugLocations : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Locations
	UPROPERTY(meta = (Input))
	TArray<FVector> Locations;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the sphere to draw
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float DrawRadius = 10.0f;

	// Number of segments in the sphere
	UPROPERTY(meta = (Input))
	int32 Segments = 8;
};


// Draws a rotation
USTRUCT(meta = (DisplayName = "Draw Debug Rotation"))
struct FRigUnit_DrawDebugRotation : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the sphere to draw
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float DrawRadius = 10.0f;
};

// Debug Draw an arrow at the given location facing in the given direction
USTRUCT(meta = (DisplayName = "Draw Debug Direction"))
struct FRigUnit_DrawDebugDirection : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
		UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Direction to Draw
	UPROPERTY(meta = (Input))
	FVector Direction = FVector::ForwardVector;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Length of the arrow to draw
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float DrawArrowLength = 10.0f;

	// Scale of the arrow head to draw
	UPROPERTY(meta = (Input))
	float ArrowHeadScale = 1.0f;
};

// Debug Draw a line at the given location scaled by the given velocity
USTRUCT(meta = (DisplayName = "Draw Debug Velocity"))
struct FRigUnit_DrawDebugVelocity : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Velocity to draw
	UPROPERTY(meta = (Input))
	FVector Velocity = FVector::ZeroVector;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Scale of the line to draw
	UPROPERTY(meta = (Input))
	float DrawVelocityLineScale = 1.0f;
};

// Debug Draw lines at the given locations scaled by the given velocities
USTRUCT(meta = (DisplayName = "Draw Debug Velocities"))
struct FRigUnit_DrawDebugVelocities : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Locations
	UPROPERTY(meta = (Input))
	TArray<FVector> Locations;

	// Velocities to draw
	UPROPERTY(meta = (Input))
	TArray<FVector> Velocities;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Scale of the line to draw
	UPROPERTY(meta = (Input))
	float DrawVelocityLineScale = 1.0f;
};


// Debug Draw a set of axes at the given transform
USTRUCT(meta = (DisplayName = "Draw Debug Transform"))
struct FRigUnit_DrawDebugTransform : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Transform to draw
	UPROPERTY(meta = (Input))
	FTransform Transform = FTransform::Identity;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Radius of the transform axis
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float DrawRadius = 10.0f;
};

// Debug draw a phase-representation of an event at a location
USTRUCT(meta = (DisplayName = "Draw Debug Event"))
struct FRigUnit_DrawDebugEvent : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// If the time until the event is known
	UPROPERTY(meta = (Input))
	bool bTimeUntilEventKnown = false;

	// The time until the event
	UPROPERTY(meta = (Input, ForceUnits = "s"))
	float TimeUntilEvent = 0.0f;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Size of the event representation to draw
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Size = 20.0f;
};

// Debug draw a basic chair shape
USTRUCT(meta = (DisplayName = "Draw Debug Chair"))
struct FRigUnit_DrawDebugChair : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Chair settings
	UPROPERTY(meta = (Input))
	FDrawDebugChairSettings Settings;
};

// Debug draw a basic door shape
USTRUCT(meta = (DisplayName = "Draw Debug Door"))
struct FRigUnit_DrawDebugDoor : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Door settings
	UPROPERTY(meta = (Input))
	FDrawDebugDoorSettings Settings;
};

// Debug draw a camera shape
USTRUCT(meta = (DisplayName = "Draw Debug Camera"))
struct FRigUnit_DrawDebugCamera : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Camera Scale
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Scale = 10.0f;

	// Camera FOV
	UPROPERTY(meta = (Input, ForceUnits = "deg"))
	float FOVDegrees = 30.0f;
};

// Debug draw a backpack
USTRUCT(meta = (DisplayName = "Draw Debug Backpack"))
struct FRigUnit_DrawDebugBackpack : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Backpack settings
	UPROPERTY(meta = (Input))
	FDrawDebugBackpackSettings Settings;
};


// Debug draw a briefcase
USTRUCT(meta = (DisplayName = "Draw Debug Briefcase"))
struct FRigUnit_DrawDebugBriefcase : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Briefcase settings
	UPROPERTY(meta = (Input))
	FDrawDebugBriefcaseSettings Settings;
};

// Debug Draw a trajectory of locations and directions
USTRUCT(meta = (DisplayName = "Draw Debug Trajectory"))
struct FRigUnit_DrawDebugTrajectory : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Trajectory Locations
	UPROPERTY(meta = (Input))
	TArray<FVector> Locations;

	// Trajectory Directions
	UPROPERTY(meta = (Input))
	TArray<FVector> Directions;

	// Relative Transform to apply to the locations and directions
	UPROPERTY(meta = (Input))
	FTransform RelativeTransform = FTransform::Identity;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Length of the arrow to draw for directions
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float DrawArrowLength = 100.0f;

	// Radius of the location spheres to draw
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float PointRadius = 100.0f;

	// Scale of the arrow head to draw
	UPROPERTY(meta = (Input))
	float ArrowHeadScale = 1.0f;

	// Number of segments in the location spheres
	UPROPERTY(meta = (Input))
	int32 Segments = 9;

	// Vertical offset to apply to the trajectory to avoid it clipping with the ground
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float VerticalOffset = 2.0f;
};

// Debug Draws a transform trajectory
USTRUCT(meta = (DisplayName = "Draw Debug Transform Trajectory"))
struct FRigUnit_DrawDebugTransformTrajectory : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Transform Trajectory
	UPROPERTY(meta = (Input))
	FTransformTrajectory TransformTrajectory;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Sample sphere radius
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 5.0f;

	// Vertical offset to apply to prevent clipping with ground plane
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float VerticalOffset = 2.0f;
};

// Draws a flat 2d motif representing mover position and rotation
USTRUCT(meta = (DisplayName = "Draw Debug Mover Orientation"))
struct FRigUnit_DrawDebugMoverOrientation : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Forward Direction Vector Location
	UPROPERTY(meta = (Input))
	FVector ForwardVector = FVector::ForwardVector;

	// Motif Scale
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Scale = 30.0f;
};

// Gets the default bone color
USTRUCT(meta = (DisplayName = "Get Default Bone Color"))
struct FRigVMFunction_DrawDebugGetDefaultBoneColor : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Default Bone Color
	UPROPERTY(meta = (Output))
	FLinearColor Color = FLinearColor(0.0f, 0.0f, 0.025f, 1.0f);
};

// Gets the default bone radius
USTRUCT(meta = (DisplayName = "Get Default Bone Radius"))
struct FRigVMFunction_DrawDebugGetDefaultBoneRadius : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Default Bone Radius
	UPROPERTY(meta = (Output))
	float Radius = 1.0f;
};

// Debug Draw a bone
USTRUCT(meta = (DisplayName = "Draw Debug Bone"))
struct FRigUnit_DrawDebugBone : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Draw Color
	UPROPERTY(meta = (Input))
	FLinearColor Color = FLinearColor(0.0f, 0.0f, 0.025f, 1.0f);

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Bone Radius
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 5.0f;

	// Number of segments
	UPROPERTY(meta = (Input))
	int32 Segments = 10;

	// If to draw the bone transform
	UPROPERTY(meta = (Input))
	bool bDrawTransform = false;
};

// Debug Draw a link between a child and parent bone
USTRUCT(meta = (DisplayName = "Draw Debug Bone Link"))
struct FRigUnit_DrawDebugBoneLink : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Child Location
	UPROPERTY(meta = (Input))
	FVector ChildLocation = FVector::ZeroVector;

	// Draw Parent Location
	UPROPERTY(meta = (Input))
	FVector ParentLocation = FVector::ZeroVector;

	// Draw Parent Rotation
	UPROPERTY(meta = (Input))
	FRotator ParentRotation = FRotator::ZeroRotator;

	// Draw Color
	UPROPERTY(meta = (Input))
	FLinearColor Color = FLinearColor(0.0f, 0.0f, 0.025f, 1.0f);

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Bone Radius
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float Radius = 5.0f;
};

// Draws a skeleton from a skinned mesh component
USTRUCT(meta = (DisplayName = "Draw Debug Skeleton from Skinned Mesh Component"))
struct FRigUnit_DrawDebugSkeletonFromSkinnedMeshComponent : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	//Skinned Mesh
	UPROPERTY(meta = (Input))
	TObjectPtr<USkinnedMeshComponent> SkinnedMeshComponent = nullptr;

	// Draw Color
	UPROPERTY(meta = (Input))
	FLinearColor Color = FLinearColor(0.0f, 0.0f, 0.025f, 1.0f);

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Bone Settings
	UPROPERTY(meta = (Input))
	FDrawDebugSkeletonSettings Settings;
};

// Debug Draw a pose made up of bone locations and velocities
USTRUCT(meta = (DisplayName = "Draw Debug Pose"))
struct FRigUnit_DrawDebugPose : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Locations
	UPROPERTY(meta = (Input))
	TArray<FVector> BoneLocations;

	// Draw Linear Velocities
	UPROPERTY(meta = (Input))
	TArray<FVector> BoneLinearVelocities;

	// Relative Transform to apply to the locations and velocities
	UPROPERTY(meta = (Input))
	FTransform RelativeTransform = FTransform::Identity;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Scale for drawing velocities
	UPROPERTY(meta = (Input))
	float DrawVelocityLineScale = 1.0f;
};


// Get the number of line segments required to draw debug a string.
USTRUCT(meta = (DisplayName = "Draw Debug String Segment Num"))
struct FRigVMFunction_DrawDebugStringSegmentNum : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// String to draw
	UPROPERTY(meta = (Input))
	FString String = TEXT("");

	// String Settings
	UPROPERTY(meta = (Input))
	FDrawDebugStringSettings Settings;

	// Output Segment Num
	UPROPERTY(meta = (Output))
	int32 SegmentNum = 0;
};

// Get the number of line segments required to draw debug a name.
USTRUCT(meta = (DisplayName = "Draw Debug Name Segment Num"))
struct FRigVMFunction_DrawDebugNameSegmentNum : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Name to draw
	UPROPERTY(meta = (Input))
	FName Name = TEXT("");

	// String Settings
	UPROPERTY(meta = (Input))
	FDrawDebugStringSettings Settings;

	// Output Segment Num
	UPROPERTY(meta = (Output))
	int32 SegmentNum = 0;
};

// Get the dimensions of a draw debug string. Useful for aligning or centering text.
USTRUCT(meta = (DisplayName = "Draw Debug String Dimensions"))
struct FRigVMFunction_DrawDebugStringDimensions : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// String to draw
	UPROPERTY(meta = (Input))
	FString String = TEXT("");

	// String Settings
	UPROPERTY(meta = (Input))
	FDrawDebugStringSettings Settings;

	// Output Dimensions
	UPROPERTY(meta = (Output))
	FVector Dimensions = FVector::ZeroVector;
};

// Get the dimensions of a draw debug name. Useful for aligning or centering text.
USTRUCT(meta = (DisplayName = "Draw Debug Name Dimensions"))
struct FRigVMFunction_DrawDebugNameDimensions : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Name to draw
	UPROPERTY(meta = (Input))
	FName Name = TEXT("");

	// String Settings
	UPROPERTY(meta = (Input))
	FDrawDebugStringSettings Settings;

	// Output Dimensions
	UPROPERTY(meta = (Output))
	FVector Dimensions = FVector::ZeroVector;
};

// Get the local offset required to center a draw debug string.
USTRUCT(meta = (DisplayName = "Draw Debug String Centering Offset"))
struct FRigVMFunction_DrawDebugStringCenteringOffset : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// String to draw
	UPROPERTY(meta = (Input))
	FString String = TEXT("");

	// String Settings
	UPROPERTY(meta = (Input))
	FDrawDebugStringSettings Settings;

	// Output Offset
	UPROPERTY(meta = (Output))
	FVector Offset = FVector::ZeroVector;
};

// Get the local offset required to center a draw debug name.
USTRUCT(meta = (DisplayName = "Draw Debug Name Centering Offset"))
struct FRigVMFunction_DrawDebugNameCenteringOffset : public FRigVMFunction_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Name to draw
	UPROPERTY(meta = (Input))
	FName Name = TEXT("");

	// String Settings
	UPROPERTY(meta = (Input))
	FDrawDebugStringSettings Settings;

	// Output Offset
	UPROPERTY(meta = (Output))
	FVector Offset = FVector::ZeroVector;
};

// Debug Draws a string 
USTRUCT(meta = (DisplayName = "Draw Debug String"))
struct FRigUnit_DrawDebugString : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// String to draw
	UPROPERTY(meta = (Input))
	FString String = TEXT("");

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// String Settings
	UPROPERTY(meta = (Input))
	FDrawDebugStringSettings Settings;
};

// Debug draws a name 
USTRUCT(meta = (DisplayName = "Draw Debug Name"))
struct FRigUnit_DrawDebugName : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Name to draw
	UPROPERTY(meta = (Input))
	FName Name = NAME_None;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// String Settings
	UPROPERTY(meta = (Input))
	FDrawDebugStringSettings Settings;
};

// Debug draws several names
USTRUCT(meta = (DisplayName = "Draw Debug Names"))
struct FRigUnit_DrawDebugNames : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Names to draw
	UPROPERTY(meta = (Input))
	TArray<FName> Names;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Separator to put between names
	UPROPERTY(meta = (Input))
	FString Separator = TEXT(", ");

	// Line Style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// String Settings
	UPROPERTY(meta = (Input))
	FDrawDebugStringSettings Settings;
};

// Debug Draw a string to the visual logger. Will do nothing if not using a VisualLoggerDrawer. Will not display on-screen during recording.
USTRUCT(meta = (DisplayName = "Visual Logger Draw String"))
struct FRigUnit_DrawDebugVisualLoggerDrawString : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// String to draw
	UPROPERTY(meta = (Input))
	FString String = TEXT("");

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Color
	UPROPERTY(meta = (Input))
	FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f);
};

// Debug Draw a name to the visual logger. Will do nothing if not using a VisualLoggerDrawer. Will not display on-screen during recording.
USTRUCT(meta = (DisplayName = "Visual Logger Draw Name"))
struct FRigUnit_DrawDebugVisualLoggerDrawName : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Name to draw
	UPROPERTY(meta = (Input))
	FName Name = NAME_None;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Color
	UPROPERTY(meta = (Input))
	FLinearColor Color = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f);
};

// Debug Draw a simple graph.
USTRUCT(meta = (DisplayName = "Draw Debug Graph"))
struct FRigUnit_DrawDebugGraph : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// X Values
	UPROPERTY(meta = (Input))
	TArray<float> Xvalues;

	// Y Values
	UPROPERTY(meta = (Input))
	TArray<float> Yvalues;

	// X min
	UPROPERTY(meta = (Input))
	float Xmin = 0.0f;

	// X max
	UPROPERTY(meta = (Input))
	float Xmax = 1.0f;

	// Y min
	UPROPERTY(meta = (Input))
	float Ymin = 0.0f;

	// Y max
	UPROPERTY(meta = (Input))
	float Ymax = 1.0f;

	// X axis length
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float XaxisLength = 100.0f;

	// Y axis length
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float YaxisLength = 100.0f;

	// Text line style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle TextLineStyle;

	// Axes line style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle AxesLineStyle;

	// Plot line style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle PlotLineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Axes Settings
	UPROPERTY(meta = (Input))
	FDrawDebugGraphAxesSettings AxesSettings;
};

// Debug Draw a simple graph axes labels.
USTRUCT(meta = (DisplayName = "Draw Debug Graph Axes Labels"))
struct FRigUnit_DrawDebugGraphAxesLabels : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// X axis length
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float XaxisLength = 100.0f;

	// Y axis length
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float YaxisLength = 100.0f;

	// Text line style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle TextLineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Axes Settings
	UPROPERTY(meta = (Input))
	FDrawDebugGraphAxesSettings AxesSettings;
};

// Debug Draw a simple graph axes.
USTRUCT(meta = (DisplayName = "Draw Debug Graph Axes"))
struct FRigUnit_DrawDebugGraphAxes : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// X axis length
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float XaxisLength = 100.0f;

	// Y axis length
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float YaxisLength = 100.0f;

	// Text line style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle TextLineStyle;

	// Axes line style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle AxesLineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Axes Settings
	UPROPERTY(meta = (Input))
	FDrawDebugGraphAxesSettings AxesSettings;
};

// Debug Draw a line on a simple graph.
USTRUCT(meta = (DisplayName = "Draw Debug Graph Line"))
struct FRigUnit_DrawDebugGraphLine : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// X Values
	UPROPERTY(meta = (Input))
	TArray<float> Xvalues;

	// Y Values
	UPROPERTY(meta = (Input))
	TArray<float> Yvalues;

	// X min
	UPROPERTY(meta = (Input))
	float Xmin = 0.0f;

	// X max
	UPROPERTY(meta = (Input))
	float Xmax = 1.0f;

	// Y min
	UPROPERTY(meta = (Input))
	float Ymin = 0.0f;

	// Y max
	UPROPERTY(meta = (Input))
	float Ymax = 1.0f;

	// X axis length
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float XaxisLength = 100.0f;

	// Y axis length
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float YaxisLength = 100.0f;

	// Plot line style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle LineStyle;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;
};


// Debug Draw a simple graph legend.
USTRUCT(meta = (DisplayName = "Draw Debug Graph Legend"))
struct FRigUnit_DrawDebugGraphLegend : public FRigUnit_DrawDebugLibraryBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Debug Drawer
	UPROPERTY(meta = (Input))
	FDebugDrawer Drawer;

	// Draw Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Draw Rotation
	UPROPERTY(meta = (Input))
	FRotator Rotation = FRotator::ZeroRotator;

	// Legend Colors
	UPROPERTY(meta = (Input))
	TArray<FLinearColor> LegendColors;

	// Legend Labels
	UPROPERTY(meta = (Input))
	TArray<FString> LegendLabels;

	// X axis length
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float XaxisLength = 100.0f;

	// Y axis length
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float YaxisLength = 100.0f;

	// Text line style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle TextLineStyle;

	// Icon line style
	UPROPERTY(meta = (Input))
	FDrawDebugLineStyle IconLineStyle;

	// Icon Size
	UPROPERTY(meta = (Input, ForceUnits = "cm"))
	float IconSize = 2.5f;

	// If to use depth testing while drawing
	UPROPERTY(meta = (Input))
	bool bDepthTest = true;

	// Legend string settings
	UPROPERTY(meta = (Input))
	FDrawDebugStringSettings LegendSettings;
};

#undef UE_API